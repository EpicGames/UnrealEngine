// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MassTests : TestModuleRules
{
	static MassTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "MassEntity";
			TestMetadata.TestShortName = "Mass Entity";
		}
	}

	public MassTests(ReadOnlyTargetRules Target) : base(Target, true)
	{
		bAllowUETypesInNamespaces = true;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"MassCore",
				"MassEntity",
			}
		);

		// Access to Internal headers (MassArchetypeData.h etc.) for low-level testing
		PrivateIncludePaths.Add(System.IO.Path.Combine(ModuleDirectory, "../../Runtime/MassEntity/Internal"));

	}
}
