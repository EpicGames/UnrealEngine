// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class MassTestsTarget : TestTargetRules
{
	public MassTestsTarget(TargetInfo Target) : base(Target)
	{
		bUsePlatformFileStub = true;
		bMockEngineDefaults = true;

		bCompileAgainstEngine = true;
		bCompileAgainstApplicationCore = true;

		bTestsRequireEngine = true;

	}
}
