// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"

#include "MassEntityBuilder.h"
#include "MassRelationCommands.h"
#include "MassExecutionContext.h"
#include "Algo/Compare.h"
#include "Relations/MassChildOf.h"

#include "TestHarness.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

using namespace UE::Mass;

//-----------------------------------------------------------------------------
// Shared base for ChildOf hierarchy tests
//-----------------------------------------------------------------------------
struct FChildOfBase : FMassLLTEntityFixture
{
	virtual ~FChildOfBase() = default;

	enum class EStructure : uint8
	{
		String,
		Tree,
		MAX
	};
	EStructure StructureType = EStructure::MAX;

	TArray<FMassEntityHandle> CreatedEntities;
	int32 NumEntities = -1;
	UMassLLTProcessorBase* Processor = nullptr;
	bool bAPISupportsReparenting = true;
	FTypeHandle RelationTypeHandle;
	UE::Mass::FRelationManager* RelationManager = nullptr;

	virtual void BuildHierarchy() = 0;
	virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex)
	{
		return false;
	}
	virtual bool DeleteEntity(const int32 EntityIndex)
	{
		EntityManager->DestroyEntity(CreatedEntities[EntityIndex]);
		return true;
	}

	virtual void SetUpRelationHandle()
	{
		RelationTypeHandle = EntityManager->GetTypeManager().GetRelationTypeHandle(FMassChildOfRelation::StaticStruct());
		CHECK(RelationTypeHandle == UE::Mass::Relations::ChildOfHandle);
	}

	void SetUpChildOfBase()
	{
		CHECK(StructureType != EStructure::MAX);
		switch (StructureType)
		{
		case EStructure::String:
			NumEntities = 5;
			break;
		case EStructure::Tree:
			NumEntities = 8;
			break;
		default:
			break;
		}

		SetUpRelationHandle();
		RelationManager = &EntityManager->GetRelationManager();

		Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
		Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Processor->EntityQuery.AddRequirement<FMassChildOfFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		Processor->EntityQuery.GroupBy(EntityManager->GetTypeManager().GetRelationTypeChecked(RelationTypeHandle).RegisteredGroupType);

		BuildHierarchy();
	}

	void ExecuteTestProcessor(TArray<int32>& Scratchpad)
	{
		Scratchpad.Reset();
		Scratchpad.AddUninitialized(CreatedEntities.Num());
		for (int32 EntityIndex = 0; EntityIndex < CreatedEntities.Num(); ++EntityIndex)
		{
			Scratchpad[EntityIndex] = -1;
		}

		EntityManager->FlushCommands();

		Processor->ForEachEntityChunkExecutionFunction = [RelationManager = RelationManager, CreatedEntitiesView = MakeArrayView(CreatedEntities), &Scratchpad](FMassExecutionContext& Context)
		{
			TConstArrayView<FMassChildOfFragment> Parents = Context.GetFragmentView<FMassChildOfFragment>();
			if (Parents.Num())
			{
				for (auto EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
				{
					const FMassEntityHandle ParentHandle = Parents[EntityIt].Parent;
					if (ParentHandle.IsValid())
					{
						const int32 EntityGlobalIndex = CreatedEntitiesView.Find(Context.GetEntity(EntityIt));
						const int32 ParentGlobalIndex = CreatedEntitiesView.Find(ParentHandle);
						REQUIRE(EntityGlobalIndex != INDEX_NONE);
					REQUIRE(ParentGlobalIndex != INDEX_NONE);
						Scratchpad[EntityGlobalIndex] = Scratchpad[ParentGlobalIndex] * 10 + EntityGlobalIndex;
					}
				}
			}
			else
			{
				// no-parent
				for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
				{
					const int32 EntityGlobalIndex = CreatedEntitiesView.Find(Context.GetEntity(EntityIndex));
					REQUIRE(EntityGlobalIndex != INDEX_NONE);
					Scratchpad[EntityGlobalIndex] = EntityGlobalIndex;
				}
			}
		};

		Processor->TestExecute(EntityManager);
	}

	void RunHierarchyTest()
	{
		TArray<int32> Scratchpad;
		ExecuteTestProcessor(Scratchpad);

		TArray<int32> ExpectedValues;

		switch (StructureType)
		{
		case EStructure::String:
			ExpectedValues = { 0, 1, 12, 123, 1234 };
			break;
		case EStructure::Tree:
			ExpectedValues = { 0, 1, 2, 13, 14, 15, 26, 27 };
			break;
		default:
			break;
		}
		INFO("Processing hierarchy produces expected results");
		CHECK(Algo::Compare(Scratchpad, ExpectedValues));

		if (bAPISupportsReparenting == false)
		{
			// the API is just for entity creation, we end the test here.
			return;
		}

		// here we reparent a leaf node
		INFO("Reparenting leaf node.");
		CHECK(ReparentEntity(CreatedEntities.Num() - 1, 0));
		switch (StructureType)
		{
		case EStructure::String:
			ExpectedValues = { 0, 1, 12, 123, 4 };
			break;
		case EStructure::Tree:
			ExpectedValues = { 0, 1, 2, 13, 14, 15, 26, 7 };
			break;
		default:
			break;
		}
		ExecuteTestProcessor(Scratchpad);
		INFO("Reparenting a leaf produces expected results");
		CHECK(Algo::Compare(Scratchpad, ExpectedValues));

		INFO("Reparenting a mid-node to a leaf node.");
		CHECK(ReparentEntity(2, 4));
		switch (StructureType)
		{
		case EStructure::String:
			ExpectedValues = { 0, 1, 42, 423, 4 };
			break;
		case EStructure::Tree:
			ExpectedValues = { 0, 1, 142, 13, 14, 15, 1426, 7 };
			break;
		default:
			break;
		}
		ExecuteTestProcessor(Scratchpad);
		INFO("Reparenting a subtree to a leaf produces expected results");
		CHECK(Algo::Compare(Scratchpad, ExpectedValues));

		INFO("Deleting a leaf node");
		CHECK(DeleteEntity(3));
		switch (StructureType)
		{
		case EStructure::String:
			ExpectedValues = { 0, 1, 42, -1, 4 };
			break;
		case EStructure::Tree:
			ExpectedValues = { 0, 1, 142, -1, 14, 15, 1426, 7 };
			break;
		default:
			break;
		}
		ExecuteTestProcessor(Scratchpad);
		INFO("Delete a leaf node produces expected results");
		CHECK(Algo::Compare(Scratchpad, ExpectedValues));

		INFO("Deleting mid-level node");
		CHECK(DeleteEntity(4));
		switch (StructureType)
		{
		case EStructure::String:
			ExpectedValues = { 0, 1, -1, -1, -1 };
			break;
		case EStructure::Tree:
			ExpectedValues = { 0, 1, -1, -1, -1, 15, -1, 7 };
			break;
		default:
			break;
		}
		ExecuteTestProcessor(Scratchpad);
		INFO("Delete a mid-level node produces expected results");
		CHECK(Algo::Compare(Scratchpad, ExpectedValues));

		INFO("Deleting top-level node");
		CHECK(DeleteEntity(0));
		switch (StructureType)
		{
		case EStructure::String:
			ExpectedValues = { -1, -1, -1, -1, -1 };
			break;
		case EStructure::Tree:
			ExpectedValues = { -1, -1, -1, -1, -1, -1, -1, -1 };
			break;
		default:
			break;
		}
		ExecuteTestProcessor(Scratchpad);
		INFO("Delete a top-level node produces expected results");
		CHECK(Algo::Compare(Scratchpad, ExpectedValues));
	}
};

//-----------------------------------------------------------------------------
// FChildOf_IndividualAPI_StringHierarchy
//-----------------------------------------------------------------------------
struct FChildOf_IndividualAPI_StringHierarchy : FChildOfBase
{
	FChildOf_IndividualAPI_StringHierarchy()
	{
		StructureType = EStructure::String;
		SetUpChildOfBase();
	}

	virtual void BuildHierarchy() override
	{
		EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);
		for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
		{
			FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
			IndexCounterFragment.Value = Index;
		}

		for (int32 ChildIndex = 1; ChildIndex < CreatedEntities.Num(); ++ChildIndex)
		{
			RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[ChildIndex], CreatedEntities[ChildIndex - 1]);
		}
	}

	virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
	{
		return RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[ChildIndex], CreatedEntities[ParentIndex]).IsValid();
	}
};

TEST_CASE_METHOD(FChildOf_IndividualAPI_StringHierarchy, "Mass::ChildOfRelation::IndividualAPI::StringHierarchy", "[Mass][ChildOfRelation]")
{
	RunHierarchyTest();
}

//-----------------------------------------------------------------------------
// FChildOf_IndividualAPI_TreeHierarchy
//-----------------------------------------------------------------------------
struct FChildOf_IndividualAPI_TreeHierarchy : FChildOfBase
{
	FChildOf_IndividualAPI_TreeHierarchy()
	{
		StructureType = EStructure::Tree;
		SetUpChildOfBase();
	}

	virtual void BuildHierarchy() override
	{
		EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);
		for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
		{
			FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
			IndexCounterFragment.Value = Index;
		}

		// [1] - child of [0]
		RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[1], CreatedEntities[0]);
		// [2] - child of [0]
		RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[2], CreatedEntities[0]);
		// [3] - child of [1]
		RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[3], CreatedEntities[1]);
		// [4] - child of [1]
		RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[4], CreatedEntities[1]);
		// [5] - child of [1]
		RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[5], CreatedEntities[1]);
		// [6] - child of [2]
		RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[6], CreatedEntities[2]);
		// [7] - child of [2]
		RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[7], CreatedEntities[2]);
	}

	virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
	{
		return RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[ChildIndex], CreatedEntities[ParentIndex]).IsValid();
	}
};

TEST_CASE_METHOD(FChildOf_IndividualAPI_TreeHierarchy, "Mass::ChildOfRelation::IndividualAPI::TreeHierarchy", "[Mass][ChildOfRelation]")
{
	RunHierarchyTest();
}

//-----------------------------------------------------------------------------
// FChildOfEntityBuilder_StringHierarchy
//-----------------------------------------------------------------------------
struct FChildOfEntityBuilder_StringHierarchy : FChildOfBase
{
	FChildOfEntityBuilder_StringHierarchy()
	{
		StructureType = EStructure::String;
		bAPISupportsReparenting = false;
		SetUpChildOfBase();
	}

	virtual void BuildHierarchy() override
	{
		FEntityBuilder Builder = EntityManager->MakeEntityBuilder();

		FTestFragment_Int& IndexCounterFragment = Builder.Add_GetRef<FTestFragment_Int>(CreatedEntities.Num());
		CreatedEntities.Add(Builder.CommitAndReprepare());

		for (int32 ChildIndex = 0; ChildIndex < 4; ++ChildIndex)
		{
			Builder.AddRelation(RelationTypeHandle, CreatedEntities.Last());
			IndexCounterFragment.Value = CreatedEntities.Num();
			CreatedEntities.Add(Builder.CommitAndReprepare());
		}
	}
};

TEST_CASE_METHOD(FChildOfEntityBuilder_StringHierarchy, "Mass::ChildOfRelation::EntityBuilder::StringHierarchy", "[Mass][ChildOfRelation]")
{
	RunHierarchyTest();
}

//-----------------------------------------------------------------------------
// FChildOfEntityBuilder_TreeHierarchy
//-----------------------------------------------------------------------------
struct FChildOfEntityBuilder_TreeHierarchy : FChildOfBase
{
	FChildOfEntityBuilder_TreeHierarchy()
	{
		StructureType = EStructure::Tree;
		bAPISupportsReparenting = false;
		SetUpChildOfBase();
	}

	virtual void BuildHierarchy() override
	{
		FEntityBuilder Builder = EntityManager->MakeEntityBuilder();

		FTestFragment_Int& IndexCounterFragment = Builder.Add_GetRef<FTestFragment_Int>(CreatedEntities.Num());
		// [0] - parent entity
		CreatedEntities.Add(Builder.CommitAndReprepare());

		Builder.AddRelation(RelationTypeHandle, CreatedEntities.Last());
		// [1] - child of [0]
		IndexCounterFragment.Value = CreatedEntities.Num();
		CreatedEntities.Add(Builder.CommitAndReprepare());
		// [2] - child of [0]
		IndexCounterFragment.Value = CreatedEntities.Num();
		CreatedEntities.Add(Builder.CommitAndReprepare());

		Builder.AddRelation(RelationTypeHandle, CreatedEntities[1]);
		// [3] - child of [1]
		IndexCounterFragment.Value = CreatedEntities.Num();
		CreatedEntities.Add(Builder.CommitAndReprepare());
		// [4] - child of [1]
		IndexCounterFragment.Value = CreatedEntities.Num();
		CreatedEntities.Add(Builder.CommitAndReprepare());
		// [5] - child of [1]
		IndexCounterFragment.Value = CreatedEntities.Num();
		CreatedEntities.Add(Builder.CommitAndReprepare());

		Builder.AddRelation(RelationTypeHandle, CreatedEntities[2]);
		// [6] - child of [2]
		IndexCounterFragment.Value = CreatedEntities.Num();
		CreatedEntities.Add(Builder.CommitAndReprepare());
		// [7] - child of [2]
		IndexCounterFragment.Value = CreatedEntities.Num();
		CreatedEntities.Add(Builder.CommitAndReprepare());
	}
};

TEST_CASE_METHOD(FChildOfEntityBuilder_TreeHierarchy, "Mass::ChildOfRelation::EntityBuilder::TreeHierarchy", "[Mass][ChildOfRelation]")
{
	RunHierarchyTest();
}

//-----------------------------------------------------------------------------
// FChildOfBatchAPI_Tree
//-----------------------------------------------------------------------------
struct FChildOfBatchAPI_Tree : FChildOfBase
{
	FChildOfBatchAPI_Tree()
	{
		StructureType = EStructure::Tree;
		SetUpChildOfBase();
	}

	virtual void BuildHierarchy() override
	{
		EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);
		for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
		{
			FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
			IndexCounterFragment.Value = Index;
		}

		// [1, 2] - child of [0]
		EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[1], 1), MakeArrayView(&CreatedEntities[0], 1));
		EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[2], 1), MakeArrayView(&CreatedEntities[0], 1));
		// [3, 4, 5] - child of [1]
		EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[3], 1), MakeArrayView(&CreatedEntities[1], 1));
		EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[4], 1), MakeArrayView(&CreatedEntities[1], 1));
		EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[5], 1), MakeArrayView(&CreatedEntities[1], 1));
		// [6, 7] - child of [2]
		EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[6], 1), MakeArrayView(&CreatedEntities[2], 1));
		EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[7], 1), MakeArrayView(&CreatedEntities[2], 1));
	}

	virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
	{
		return EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[ChildIndex], 1), MakeArrayView(&CreatedEntities[ParentIndex], 1));
	}
};

TEST_CASE_METHOD(FChildOfBatchAPI_Tree, "Mass::ChildOfRelation::BatchAPI", "[Mass][ChildOfRelation]")
{
	RunHierarchyTest();
}

//-----------------------------------------------------------------------------
// FChildOfCommands_StringHierarchy
//-----------------------------------------------------------------------------
struct FChildOfCommands_StringHierarchy : FChildOfBase
{
	FChildOfCommands_StringHierarchy()
	{
		StructureType = EStructure::String;
		SetUpChildOfBase();
	}

	virtual void BuildHierarchy() override
	{
		EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);
		for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
		{
			FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
			IndexCounterFragment.Value = Index;
			if (Index > 0)
			{
				// issue relationship-creating commands, done here for convenience
				EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(CreatedEntities[Index], CreatedEntities[Index-1]);
			}
		}

		EntityManager->FlushCommands();
	}

	virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
	{
		EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(CreatedEntities[ChildIndex], CreatedEntities[ParentIndex]);
		return true;
	}

	virtual bool DeleteEntity(const int32 EntityIndex) override
	{
		EntityManager->Defer().DestroyEntity(CreatedEntities[EntityIndex]);
		return true;
	}
};

TEST_CASE_METHOD(FChildOfCommands_StringHierarchy, "Mass::ChildOfRelation::Commands::StringHierarchy", "[Mass][ChildOfRelation]")
{
	RunHierarchyTest();
}

//-----------------------------------------------------------------------------
// FChildOfCommands_TreeHierarchy
//-----------------------------------------------------------------------------
struct FChildOfCommands_TreeHierarchy : FChildOfBase
{
	FChildOfCommands_TreeHierarchy()
	{
		StructureType = EStructure::Tree;
		SetUpChildOfBase();
	}

	virtual void BuildHierarchy() override
	{
		EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);

		for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
		{
			FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
			IndexCounterFragment.Value = Index;
		}

		// [1, 2] - child of [0]
		EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(MakeArrayView(&CreatedEntities[1], 2), MakeArrayView(&CreatedEntities[0], 1));
		// [3, 4, 5] - child of [1]
		EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(MakeArrayView(&CreatedEntities[3], 3), MakeArrayView(&CreatedEntities[1], 1));
		// [6, 7] - child of [2]
		EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(MakeArrayView(&CreatedEntities[6], 2), MakeArrayView(&CreatedEntities[2], 1));

		EntityManager->FlushCommands();
	}

	virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
	{
		EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(CreatedEntities[ChildIndex], CreatedEntities[ParentIndex]);
		EntityManager->FlushCommands();
		return true;
	}

	virtual bool DeleteEntity(const int32 EntityIndex) override
	{
		EntityManager->Defer().DestroyEntity(CreatedEntities[EntityIndex]);
		return true;
	}
};

TEST_CASE_METHOD(FChildOfCommands_TreeHierarchy, "Mass::ChildOfRelation::Commands::TreeHierarchy", "[Mass][ChildOfRelation]")
{
	RunHierarchyTest();
}

//-----------------------------------------------------------------------------
// FSetChildAsParent
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::ChildOfRelation::SetChildAsParent", "[Mass][ChildOfRelation]")
{
	FTypeHandle RelationTypeHandle = EntityManager->GetTypeManager().GetRelationTypeHandle(FMassChildOfRelation::StaticStruct());

	TArray<FMassEntityHandle> CreatedEntities;
	EntityManager->BatchCreateEntities(IntsArchetype, {}, 2, CreatedEntities);

	UE::Mass::FRelationManager& RelationManager = EntityManager->GetRelationManager();
	RelationManager.CreateRelationInstance(RelationTypeHandle, CreatedEntities[0], CreatedEntities[1]);
	EntityManager->FlushCommands();

	// original parent becomes the child and vice versa:
	RelationManager.CreateRelationInstance(RelationTypeHandle, CreatedEntities[1], CreatedEntities[0]);
	EntityManager->FlushCommands();

	TArray<FMassEntityHandle> ParentSubjects = RelationManager.GetRelationSubjects(RelationTypeHandle, CreatedEntities[0]);
	TArray<FMassEntityHandle> ParentObjects = RelationManager.GetRelationSubjects(RelationTypeHandle, CreatedEntities[0]);
	TArray<FMassEntityHandle> ChildSubjects = RelationManager.GetRelationSubjects(RelationTypeHandle, CreatedEntities[1]);
	TArray<FMassEntityHandle> ChildObjects = RelationManager.GetRelationSubjects(RelationTypeHandle, CreatedEntities[1]);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
