// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassArchetypeData.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"
#include "Math/RandomStream.h"
#include "Algo/RandomShuffle.h"
#include "Algo/Compare.h"
#include "Algo/RemoveIf.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

// Extended fixture that pre-creates 100 entities in FloatsArchetype
struct FChunkCollectionFixture : FMassLLTEntityFixture
{
	TArray<FMassEntityHandle> Entities;

	FChunkCollectionFixture()
	{
		const int32 Count = 100;
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);
	}
};

TEST_CASE_METHOD(FChunkCollectionFixture, "Mass::ArchetypeEntityCollection::Create::Basic", "[Mass][ArchetypeEntityCollection]")
{
	TArray<FMassEntityHandle> EntitiesSubSet;

	// Should end up as last chunk
	EntitiesSubSet.Add(Entities[99]);
	EntitiesSubSet.Add(Entities[97]);
	EntitiesSubSet.Add(Entities[98]);

	// Should end up as third chunk
	EntitiesSubSet.Add(Entities[20]);
	EntitiesSubSet.Add(Entities[22]);
	EntitiesSubSet.Add(Entities[21]);

	// Should end up as second chunk
	EntitiesSubSet.Add(Entities[18]);

	// Should end up as first chunk
	EntitiesSubSet.Add(Entities[10]);
	EntitiesSubSet.Add(Entities[13]);
	EntitiesSubSet.Add(Entities[11]);
	EntitiesSubSet.Add(Entities[12]);

	FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates);
	FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = EntityCollection.GetRanges();
	INFO("The predicted sub-chunk count should match");
	REQUIRE(Ranges.Num() == 4);
	INFO("The [10-13] chunk should be first and start at 10");
	CHECK(Ranges[0].SubchunkStart == 10);
	INFO("The [10-13] chunk should be first and have a length of 4");
	CHECK(Ranges[0].Length == 4);
	INFO("The [18] chunk should be second and start at 18");
	CHECK(Ranges[1].SubchunkStart == 18);
	INFO("The [18] chunk should be second and have a length of 1");
	CHECK(Ranges[1].Length == 1);
	INFO("The [20-22] chunk should be third and start at 20");
	CHECK(Ranges[2].SubchunkStart == 20);
	INFO("The [20-22] chunk should be third and have a length of 3");
	CHECK(Ranges[2].Length == 3);
	INFO("The [97-99] chunk should be fourth and start at 97");
	CHECK(Ranges[3].SubchunkStart == 97);
	INFO("The [97-99] chunk should be fourth and have a length of 3");
	CHECK(Ranges[3].Length == 3);
}

TEST_CASE_METHOD(FChunkCollectionFixture, "Mass::ArchetypeEntityCollection::Create::OrderInvariant", "[Mass][ArchetypeEntityCollection]")
{
	TArray<FMassEntityHandle> EntitiesSubSet(&Entities[10], 30);
	EntitiesSubSet.RemoveAt(10, EAllowShrinking::No);

	FMassArchetypeEntityCollection CollectionFromOrdered(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates);

	FRandomStream Rand(0);
	// Fisher-Yates shuffle using FRandomStream (inlined from MassEntityTestTypes.h)
	for (int32 Index = 0; Index < EntitiesSubSet.Num(); ++Index)
	{
		const int32 NewIndex = Rand.RandRange(0, EntitiesSubSet.Num() - 1);
		EntitiesSubSet.Swap(Index, NewIndex);
	}

	FMassArchetypeEntityCollection CollectionFromRandom(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates);

	INFO("The resulting chunk collection should be the same regardless of the order of input entities");
	CHECK(CollectionFromOrdered.IsSame(CollectionFromRandom));

	// just to roughly make sure the result is what we expect
	FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = CollectionFromOrdered.GetRanges();
	INFO("The result should contain two chunks");
	CHECK(Ranges.Num() == 2);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::ArchetypeEntityCollection::Create::CrossChunk", "[Mass][ArchetypeEntityCollection]")
{
#if WITH_MASSENTITY_DEBUG
	TArray<FMassEntityHandle> Entities;
	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);

	const int32 SpillOver = 10;
	const int32 Count = EntitiesPerChunk + SpillOver;
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

	TArray<FMassEntityHandle> EntitiesSubCollection;
	EntitiesSubCollection.Add(Entities[EntitiesPerChunk]);
	for (int32 Index = 1; Index < SpillOver; ++Index)
	{
		EntitiesSubCollection.Add(Entities[EntitiesPerChunk + Index]);
		EntitiesSubCollection.Add(Entities[EntitiesPerChunk - Index]);
	}

	FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesSubCollection, FMassArchetypeEntityCollection::NoDuplicates);
	FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = EntityCollection.GetRanges();
	INFO("The given continuous range should get split in two");
	REQUIRE(Ranges.Num() == 2);
	INFO("The part in first archetype's chunk should contain 9 elements");
	CHECK(Ranges[0].Length == 9);
	INFO("The part in second archetype's chunk should contain 10 elements");
	CHECK(Ranges[1].Length == 10);
#endif // WITH_MASSENTITY_DEBUG
}

TEST_CASE_METHOD(FChunkCollectionFixture, "Mass::ArchetypeEntityCollection::Create::TrivialDuplicates", "[Mass][ArchetypeEntityCollection]")
{
	TArray<FMassEntityHandle> EntitiesWithDuplicates;
	EntitiesWithDuplicates.Add(Entities[2]);
	EntitiesWithDuplicates.Add(Entities[2]);

	FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesWithDuplicates, FMassArchetypeEntityCollection::FoldDuplicates);
	FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = EntityCollection.GetRanges();
	INFO("The result should have a single subchunk");
	REQUIRE(Ranges.Num() == 1);
	INFO("The resulting subchunk should be of length 1");
	CHECK(Ranges[0].Length == 1);
}

TEST_CASE_METHOD(FChunkCollectionFixture, "Mass::ArchetypeEntityCollection::Create::Duplicates", "[Mass][ArchetypeEntityCollection]")
{
	TArray<FMassEntityHandle> EntitiesWithDuplicates;
	EntitiesWithDuplicates.Add(Entities[0]);
	EntitiesWithDuplicates.Add(Entities[0]);
	EntitiesWithDuplicates.Add(Entities[0]);
	EntitiesWithDuplicates.Add(Entities[1]);
	EntitiesWithDuplicates.Add(Entities[2]);
	EntitiesWithDuplicates.Add(Entities[2]);

	FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesWithDuplicates, FMassArchetypeEntityCollection::FoldDuplicates);
	FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = EntityCollection.GetRanges();
	INFO("The result should have a single subchunk");
	REQUIRE(Ranges.Num() == 1);
	INFO("The resulting subchunk should be of length 3");
	CHECK(Ranges[0].Length == 3);
}

TEST_CASE_METHOD(FChunkCollectionFixture, "Mass::ArchetypeEntityCollection::Create::InvalidDuplicates", "[Mass][ArchetypeEntityCollection]")
{
	{
		TArray<FMassEntityHandle> EntitiesSubSet;

		EntitiesSubSet.Add(FMassEntityHandle());
		EntitiesSubSet.Add(Entities[0]);
		EntitiesSubSet.Add(Entities[0]);
		EntitiesSubSet.Add(FMassEntityHandle());

		FMassArchetypeEntityCollection Collection(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates);

		INFO("We expect only a single resulting range");
		REQUIRE(Collection.GetRanges().Num() == 1);
		INFO("We expect the range to start at 0");
		CHECK(Collection.GetRanges()[0].SubchunkStart == 0);
		INFO("We expect only a single entity in the resulting range");
		CHECK(Collection.GetRanges()[0].Length == 1);
	}

	{
		TArray<FMassEntityHandle> EntitiesSubSet;

		EntitiesSubSet.Add(Entities[4]);
		EntitiesSubSet.Add(FMassEntityHandle());
		EntitiesSubSet.Add(FMassEntityHandle());
		EntitiesSubSet.Add(Entities[3]);
		EntitiesSubSet.Add(FMassEntityHandle());
		EntitiesSubSet.Add(Entities[1]);

		FMassArchetypeEntityCollection Collection(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates);

		INFO("We expect two resulting ranges");
		REQUIRE(Collection.GetRanges().Num() == 2);
		INFO("We expect the first range to consist of a single entity");
		CHECK(Collection.GetRanges()[0].Length == 1);
		INFO("We expect the second range to consist of two entities");
		CHECK(Collection.GetRanges()[1].Length == 2);
	}
}

// @todo Re-enable once test body is implemented. Original used MASS_SCOPED_ENSURE_TEST("Invalid entity handle passed in", 2).
DISABLED_TEST_CASE_METHOD(FChunkCollectionFixture, "Mass::ArchetypeEntityCollection::Create::InvalidDuplicatesWithPayload", "[Mass][ArchetypeEntityCollection]")
{
}

#if WITH_MASSENTITY_DEBUG
// Extended fixture that creates entities spanning 2 chunks
struct FChunkCollectionPayloadFixture : FMassLLTEntityFixture
{
	TArray<FMassEntityHandle> Entities;

	FChunkCollectionPayloadFixture()
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);
		EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesPerChunk * 2, Entities);
	}
};

TEST_CASE_METHOD(FChunkCollectionPayloadFixture, "Mass::ArchetypeEntityCollection::Create::TrivialDuplicatesWithPayload", "[Mass][ArchetypeEntityCollection][Debug]")
{
	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);
	TArray<FMassEntityHandle> EntitiesSubSet;
	TArray<FTestFragment_Int> Payload;

	EntitiesSubSet.Add(Entities[EntitiesPerChunk + 20]);
	Payload.Add(FTestFragment_Int(0));
	EntitiesSubSet.Add(Entities[EntitiesPerChunk + 20]);
	Payload.Add(FTestFragment_Int(1));

	FStructArrayView PaloadView(Payload);
	TArray<FMassArchetypeEntityCollectionWithPayload> Result;
	FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(*EntityManager, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates
		, FMassGenericPayloadView(MakeArrayView(&PaloadView, 1)), Result);

	INFO("We expect only a single result");
	REQUIRE(Result.Num() == 1);
	INFO("We expect only a single resulting range");
	REQUIRE(Result[0].GetEntityCollection().GetRanges().Num() == 1);
	INFO("We expect only a single entity in the resulting range");
	CHECK(Result[0].GetEntityCollection().GetRanges()[0].Length == 1);
}

TEST_CASE_METHOD(FChunkCollectionPayloadFixture, "Mass::ArchetypeEntityCollection::Create::MultiDuplicatesWithPayload", "[Mass][ArchetypeEntityCollection][Debug]")
{
	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);
	TArray<FMassEntityHandle> EntitiesSubSet;
	TArray<FTestFragment_Int> Payload;

	constexpr int32 NumUniques = 3;
	constexpr int32 NumDuplicatesEach = 4;
	int32 FragmentValue = 0;
	for (int32 Iteration = 0; Iteration < NumDuplicatesEach; ++Iteration, ++FragmentValue)
	{
		for (int32 Unique = 0; Unique < NumUniques; ++Unique)
		{
			EntitiesSubSet.Add(Entities[EntitiesPerChunk + 20 + Unique]);
			Payload.Add(FTestFragment_Int(Unique));
		}
	}

	FStructArrayView PaloadView(Payload);
	TArray<FMassArchetypeEntityCollectionWithPayload> Result;
	FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(*EntityManager, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates
		, FMassGenericPayloadView(MakeArrayView(&PaloadView, 1)), Result);

	INFO("We expect only a single result");
	REQUIRE(Result.Num() == 1);
	INFO("We expect only a single resulting range");
	REQUIRE(Result[0].GetEntityCollection().GetRanges().Num() == 1);
	INFO("We expect exactly NumUniques entities in the resulting range");
	CHECK(Result[0].GetEntityCollection().GetRanges()[0].Length == NumUniques);
	const FMassGenericPayloadViewSlice& PayloadSlice = Result[0].GetPayload();
	for (int32 Unique = 0; Unique < NumUniques; ++Unique)
	{
		INFO("The surviving payload value should match the expected");
		CHECK(PayloadSlice[0].GetAt<FTestFragment_Int>(Unique).Value == Unique);
	}
}

TEST_CASE_METHOD(FChunkCollectionPayloadFixture, "Mass::ArchetypeEntityCollection::Create::WithPayload", "[Mass][ArchetypeEntityCollection][Debug]")
{
	const int32 TotalCount = Entities.Num();
	const int32 SubSetCount = static_cast<int32>(0.6 * Entities.Num());
	TArray<FMassEntityHandle> EntitiesSubSet;
	TArray<FTestFragment_Int> Payload;

	TArray<int32> Indices;
	Indices.AddUninitialized(TotalCount);
	for (int32 Index = 0; Index < Indices.Num(); ++Index)
	{
		Indices[Index] = Index;
	}

	FMath::SRandInit(TotalCount);
	Algo::RandomShuffle(Indices);
	Indices.SetNum(SubSetCount);

	for (int32 i : Indices)
	{
		EntitiesSubSet.Add(Entities[i]);
		Payload.Add(FTestFragment_Int(i));
	}

	FStructArrayView PaloadView(Payload);
	TArray<FMassArchetypeEntityCollectionWithPayload> Result;
	FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(*EntityManager, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates
		, FMassGenericPayloadView(MakeArrayView(&PaloadView, 1)), Result);

	// at this point Payload should be sorted ascending
	for (int32 Index = 1; Index < Payload.Num(); ++Index)
	{
		INFO("Items in Payload should be arranged in an ascending manner");
		CHECK(Payload[Index].Value >= Payload[Index-1].Value);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::ZeroLengthRanges::Destroy", "[Mass][ArchetypeEntityCollection][Debug]")
{
	REQUIRE(EntityManager);
	const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
	const int32 NumChunks = 3;
	const int32 EntityCount = NumChunks * EntitiesPerChunk;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);

	INFO("Entity creation generated expected number of entities");
	CHECK(EntityCount == FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities());

	FMassEntityQuery Query(EntityManager);
	Query.CacheArchetypes();
	FMassArchetypeEntityCollection Collection(SourceArchetype);
	INFO("Created collection has expected number of ranges");
	CHECK(Collection.GetRanges().Num() == NumChunks);

	EntityManager->BatchDestroyEntityChunks(Collection);
	INFO("Entity destruction destroyed all entities");
	CHECK(FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities() == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::ZeroLengthRanges::Execute", "[Mass][ArchetypeEntityCollection][Debug]")
{
	REQUIRE(EntityManager);
	const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
	const int32 NumChunks = 3;
	const int32 EntityCount = NumChunks * EntitiesPerChunk;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);
	FMassArchetypeEntityCollection Collection(SourceArchetype);

	INFO("Entity creation generated expected number of entities");
	CHECK(EntityCount == FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities());

	int32 ProcessedEntitiesCount = 0;
	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	Processor->ForEachEntityChunkExecutionFunction = [&ProcessedEntitiesCount](FMassExecutionContext& Context)
	{
		ProcessedEntitiesCount += Context.GetNumEntities();
	};
	Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
	UE::Mass::Executor::RunProcessorsView(MakeArrayView((UMassProcessor**)&Processor, 1), ProcessingContext, MakeConstArrayView(&Collection, 1));

	INFO("Number of entities processed matches expectations");
	CHECK(ProcessedEntitiesCount == EntityCount);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::ZeroLengthRanges::BatchMove", "[Mass][ArchetypeEntityCollection][Debug]")
{
	REQUIRE(EntityManager);
	const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
	const int32 NumChunks = 3;
	const int32 EntityCount = NumChunks * EntitiesPerChunk;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);
	FMassArchetypeEntityCollection Collection(SourceArchetype);

	const UScriptStruct* AddedTag = FTestTag_A::StaticStruct();

	INFO("Entity creation generated expected number of entities");
	CHECK(EntityCount == FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities());
	CHECK(EntityCount == Entities.Num());

	EntityManager->BatchChangeTagsForEntities(MakeArrayView(&Collection, 1), FMassTagBitSet(AddedTag), {});

	const FMassArchetypeHandle NewArchetype = EntityManager->GetArchetypeForEntity(Entities[0]);
	INFO("Entities have changed their host archetype");
	CHECK_FALSE(NewArchetype == SourceArchetype);

	const FMassArchetypeHandle ExpectedArchetype = EntityManager->CreateArchetype(SourceArchetype, MakeArrayView(&AddedTag, 1));
	INFO("The new archetype matches expectations");
	CHECK(NewArchetype == ExpectedArchetype);
	INFO("All the entities have been moved to the expected archetype");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(NewArchetype) == EntityCount);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::ZeroLengthRanges::BatchSetFragmentValues", "[Mass][ArchetypeEntityCollection][Debug]")
{
	REQUIRE(EntityManager);
	const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
	const int32 NumChunks = 3;
	const int32 EntityCount = NumChunks * EntitiesPerChunk;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);

	INFO("Entity creation generated expected number of entities");
	CHECK(EntityCount == FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities());
	CHECK(EntityCount == Entities.Num());

	FMassArchetypeEntityCollectionWithPayload EntityCollectionWithEmptyPayload{FMassArchetypeEntityCollection(SourceArchetype)};

	const FMassFragmentBitSet AddedFragmentsBitSet(FTestFragment_Int::StaticStruct());
	EntityManager->BatchAddFragmentInstancesForEntities(MakeArrayView(&EntityCollectionWithEmptyPayload, 1), AddedFragmentsBitSet);

	const FMassArchetypeHandle NewArchetype = EntityManager->GetArchetypeForEntity(Entities[0]);
	INFO("Entities have changed their host archetype");
	CHECK_FALSE(NewArchetype == SourceArchetype);

	const FMassArchetypeHandle ExpectedArchetype = FloatsIntsArchetype;
	INFO("The new archetype matches expectations");
	CHECK(NewArchetype == ExpectedArchetype);
	INFO("All the entities have been moved to the expected archetype");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(NewArchetype) == EntityCount);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::ArchetypeEntityCollection::ExportHandles::CreatedEntities", "[Mass][ArchetypeEntityCollection][Debug]")
{
	REQUIRE(EntityManager);
	const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
	const int32 NumChunks = 3;
	const int32 EntityCount = NumChunks * EntitiesPerChunk;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);

	// deterministically removing half of the entities
	FRandomStream RandomStream(0);
	const int32 RemainingCount = Algo::StableRemoveIf(Entities, [&RandomStream](const FMassEntityHandle& Element)
	{
		return RandomStream.FRand() < 0.5f;
	});
	Entities.SetNum(RemainingCount);

	FMassArchetypeEntityCollection Collection(SourceArchetype, Entities, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);

	TArray<FMassEntityHandle> ExportedHandles;
	Collection.ExportEntityHandles(ExportedHandles);

	INFO("Exported handles are the same as the input data");
	CHECK(Algo::Compare(ExportedHandles, Entities));
}

#endif // WITH_MASSENTITY_DEBUG

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
