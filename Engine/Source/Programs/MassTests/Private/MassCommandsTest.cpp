// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"

#include "MassEntityManager.h"
#include "MassCommands.h"
#include "MassProcessingTypes.h"
#include "MassEntityTypes.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassObserverManager.h"
#include "Algo/Sort.h"
#include "Algo/RandomShuffle.h"

#include "TestHarness.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

#if WITH_MASSENTITY_DEBUG

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Commands::FragmentInstanceList", "[Mass][Commands][Debug]")
{
	const int32 Count = 5;
	TArray<FMassEntityHandle> IntEntities;
	TArray<FMassEntityHandle> FloatEntities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, IntEntities);
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, FloatEntities);

	for (int32 Index = 0; Index < Count; ++Index)
	{
		EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(IntEntities[Index], FTestFragment_Int(Index), FTestFragment_Float(static_cast<float>(Index)));
		EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(FloatEntities[Index], FTestFragment_Int(Index), FTestFragment_Float(static_cast<float>(Index)));
	}

	EntityManager->FlushCommands();

	auto TestEntities = [this](const TArray<FMassEntityHandle>& Entities)
	{
		// all entities should have ended up in the same archetype, FloatsIntsArchetype
		for (int32 Index = 0; Index < Entities.Num(); ++Index)
		{
			INFO("All entities should have ended up in the same archetype");
			CHECK(EntityManager->GetArchetypeForEntity(Entities[Index]) == FloatsIntsArchetype);

			FMassEntityView View(FloatsIntsArchetype, Entities[Index]);
			INFO("Should have predicted values");
			CHECK(View.GetFragmentData<FTestFragment_Int>().Value == Index);
			INFO("Should have predicted values");
			CHECK(View.GetFragmentData<FTestFragment_Float>().Value == static_cast<float>(Index));
		}
	};

	TestEntities(IntEntities);
	TestEntities(FloatEntities);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Commands::MemoryManagement", "[Mass][Commands][Debug]")
{
	const UScriptStruct* ArrayFragmentTypes[] = {
		FTestFragment_Array::StaticStruct(),
		FTestFragment_Int::StaticStruct()
	};
	const FMassArchetypeHandle ArrayArchetype = EntityManager->CreateArchetype(MakeArrayView(ArrayFragmentTypes, 1));
	const FMassArchetypeHandle ArrayIntArchetype = EntityManager->CreateArchetype(MakeArrayView(ArrayFragmentTypes, 2));
	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(ArrayArchetype);
	const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(ArrayArchetype, Count, Entities);

	INFO("All entities created should be in ArrayArchetype");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(ArrayArchetype) == Entities.Num());

	TArray<int32> EntitiesWithArray;
	for (int32 EntityIndex = 0; EntityIndex < Count; ++EntityIndex)
	{
		if (FMath::FRand() < 0.2)
		{
			FTestFragment_Array A;
			A.Value.Add(EntityIndex);
			EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[EntityIndex], A);
			EntityManager->Defer().AddFragment<FTestFragment_Int>(Entities[EntityIndex]);
			EntitiesWithArray.Add(EntityIndex);
		}
	}

	EntityManager->FlushCommands();

	for (int32 EntityIndex : EntitiesWithArray)
	{
		FMassEntityView View(ArrayIntArchetype, Entities[EntityIndex]);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Array>().Value.Num() == 1);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Array>().Value[0] == EntityIndex);
	}

	// now move things around by adding yet another fragment. That will force moving of some array-hosting fragments
	for (int32 EntityIndex = 0; EntityIndex < Count; ++EntityIndex)
	{
		if (FMath::FRand() < 0.5)
		{
			EntityManager->Defer().AddFragment<FTestFragment_Float>(Entities[EntityIndex]);
		}
	}

	EntityManager->FlushCommands();

	for (int32 EntityIndex : EntitiesWithArray)
	{
		FMassEntityView View(*EntityManager, Entities[EntityIndex]);
		INFO("Potentially moved array fragment should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Array>().Value.Num() == 1);
		INFO("Potentially moved array fragment should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Array>().Value[0] == EntityIndex);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Commands::BuildEntitiesWithFragments", "[Mass][Commands][Debug]")
{
	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
	const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

	TArray<FMassEntityHandle> Entities;
	for (int32 Index = 0; Index < Count; ++Index)
	{
		Entities.Add(EntityManager->ReserveEntity());
		EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities.Last(), FTestFragment_Int(Index), FTestFragment_Float(static_cast<float>(Index)));
	}

	INFO("All entities created should be in FloatsIntsArchetype before flush");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == 0);
	EntityManager->FlushCommands();
	INFO("All entities created should be in FloatsIntsArchetype after flush");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == Entities.Num());

	for (int32 Index = 0; Index < Entities.Num(); ++Index)
	{
		FMassEntityView View(FloatsIntsArchetype, Entities[Index]);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Int>().Value == Index);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Float>().Value == static_cast<float>(Index));
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Commands::BuildEntitiesInHoles", "[Mass][Commands][Debug]")
{
	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
	const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 1.25f) * 2; // making sure it's even

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);
	FMath::SRandInit(0);
	Algo::RandomShuffle(Entities);
	EntityManager->BatchDestroyEntities(MakeArrayView(Entities.GetData(), Entities.Num()/2));

	Entities.Reset();
	for (int32 Index = 0; Index < EntitiesPerChunk; ++Index)
	{
		Entities.Add(EntityManager->ReserveEntity());
		EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities.Last(), FTestFragment_Int(Index), FTestFragment_Float(static_cast<float>(Index)));
	}

	INFO("All entities created should be in FloatsIntsArchetype before flush");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == Count / 2);
	EntityManager->FlushCommands();
	INFO("All entities created should be in FloatsIntsArchetype after flush");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == Count / 2 + Entities.Num());

	for (int32 Index = 0; Index < Entities.Num(); ++Index)
	{
		FMassEntityView View(FloatsIntsArchetype, Entities[Index]);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Int>().Value == Index);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Float>().Value == static_cast<float>(Index));
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Commands::BuildEntitiesWithFragmentInstances", "[Mass][Commands][Debug]")
{
	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
	const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

	TArray<FMassEntityHandle> Entities;
	for (int32 Index = 0; Index < Count; ++Index)
	{
		Entities.Add(EntityManager->ReserveEntity());
		EntityManager->Defer().PushCommand<FMassCommandBuildEntity>(Entities.Last(), FTestFragment_Int(Index), FTestFragment_Float(static_cast<float>(Index)));
	}

	INFO("All entities created should be in FloatsIntsArchetype before flush");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == 0);
	EntityManager->FlushCommands();
	INFO("All entities created should be in FloatsIntsArchetype after flush");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == Entities.Num());

	for (int32 Index = 0; Index < Entities.Num(); ++Index)
	{
		FMassEntityView View(FloatsIntsArchetype, Entities[Index]);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Int>().Value == Index);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Float>().Value == static_cast<float>(Index));
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Commands::DeferredFunction", "[Mass][Commands][Debug]")
{
	constexpr int32 Count = 5;
	const int32 Offset = 1000;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

	int32 EntityIndex = 0;
	for (FMassEntityHandle Entity : Entities)
	{
		FMassEntityView View(IntsArchetype, Entity);
		View.GetFragmentData<FTestFragment_Int>().Value = Offset + EntityIndex++;

		EntityManager->Defer().PushCommand<FMassDeferredSetCommand>([Entity, Archetype = IntsArchetype, Offset](FMassEntityManager&)
			{
				FMassEntityView View(Archetype, Entity);
				View.GetFragmentData<FTestFragment_Int>().Value -= Offset;
			});
	}

	EntityManager->FlushCommands();

	for (EntityIndex = 0; EntityIndex < Entities.Num(); ++EntityIndex)
	{
		FMassEntityView View(IntsArchetype, Entities[EntityIndex]);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Int>().Value == EntityIndex);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Commands::PushWhileFlushing", "[Mass][Commands][Debug]")
{
	constexpr int32 Count = 5;

	// here's what we want to do:
	// 1. Create a Count number of Int entities
	// 2. Register TagA observer that will add a float fragment when the tag is added
	//	a. The observer will use EntityManager.Defer() directly for the testing purposes - it should use Context.Defer() in real world scenarios
	// 3. Add TagA to all the created Entities
	// 4. Test if all the affected entities have the float fragment after the flushing

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);
	for (const FMassEntityHandle& EntityHandle : Entities)
	{
		INFO("None of the freshly created entities is expected to contain a float fragment");
		CHECK(EntityManager->GetFragmentDataPtr<FTestFragment_Float>(EntityHandle) == nullptr);
	}

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context)
		{
			for (const FMassEntityHandle& EntityHandle : Context.GetEntities())
			{
				Context.GetEntityManagerChecked().Defer().AddFragment<FTestFragment_Float>(EntityHandle);
			}
		};
	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	EntityManager->Defer().PushCommand<FMassCommandAddTag<FTestTag_A>>(Entities);
	for (const FMassEntityHandle& EntityHandle : Entities)
	{
		INFO("Pushing the AddTag command should not result in adding the float fragment");
		CHECK(EntityManager->GetFragmentDataPtr<FTestFragment_Float>(EntityHandle) == nullptr);
	}

	EntityManager->FlushCommands();

	for (const FMassEntityHandle& EntityHandle : Entities)
	{
		INFO("After flushing all the observed entities should have the float fragment");
		CHECK(EntityManager->GetFragmentDataPtr<FTestFragment_Float>(EntityHandle) != nullptr);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Commands::MoveHandleArrays", "[Mass][Commands][Debug]")
{
	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
	const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

	TArray<FMassArchetypeEntityCollection> EntityCollections;
	UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
	EntityManager->BatchChangeTagsForEntities(EntityCollections, FMassTagBitSet(FTestTag_A::StaticStruct()), FMassTagBitSet());

	// verify that original archetypes no longer host any entities
	INFO("Original archetypes are empty after adding a tag to all entities");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 0);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);

	EntityManager->Defer().PushCommand<FMassCommandRemoveTag<FTestTag_A>>(MoveTemp(Entities));
	EntityManager->FlushCommands();

	INFO("All the entities moved back to the original archetypes");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == Count);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == Count);
}

#endif // WITH_MASSENTITY_DEBUG

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
