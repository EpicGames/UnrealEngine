// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityManager.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassProcessingPhaseManager.h"
#include "MassCommands.h"
#include "HAL/IConsoleManager.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

//-----------------------------------------------------------------------------
// Testing the issue first reported here https://forums.unrealengine.com/t/masscommandaddfragmentinstances-adding-fragments-to-incorrect-entities/731341
//
// The issue boiled down to FMassArchetypeData::BatchMoveEntitiesToAnotherArchetype was moving entities to the new
// archetype in a changed order, while the call site (FMassEntityManager::BatchAddFragmentInstancesForEntities) the
// order has not changed and subsequently assigned mismatched values to the freshly moved entities.
//
// The fix was splitting BatchMoveEntitiesToAnotherArchetype into two sections:
// 1. adding all the new entities to the new archetype (in original order)
// 2. removal from the previous archetype (back-to-front to keep indices relevant)
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Complex::AddingFragmentInstancesToDiscontinuousEntitiesCollection", "[Mass][Complex]")
{
	constexpr int32 EntitiesCount = 4;
	constexpr int32 MiddleIndex = EntitiesCount / 2;
	REQUIRE(EntityManager);

	// Create EntitiesCount entities containing only a FTestFragment_Int fragment
	// and set the value to subsequent integer values
	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(IntsArchetype, EntitiesCount, EntitiesCreated);

	FMassExecutionContext ExecContext(*EntityManager.Get());
	FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Int::StaticStruct() });
	Query.ForEachEntityChunk(ExecContext, [](FMassExecutionContext& Context)
		{
			TArrayView<FTestFragment_Int> Ints = Context.GetMutableFragmentView<FTestFragment_Int>();
			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				Ints[EntityIndex].Value = EntityIndex;
			}
		});

	for (int32 EntityIndex = EntitiesCount - 1; EntityIndex >= 0; --EntityIndex)
	{
		// skipping one index somewhere in the middle to ensure discontinuity in entities being processed by the command
		if (EntityIndex != MiddleIndex)
		{
			const FTestFragment_Int& IntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(EntitiesCreated[EntityIndex]);
			EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(EntitiesCreated[EntityIndex], FTestFragment_Float(static_cast<float>(IntFragment.Value)));
		}
	}
	EntityManager->FlushCommands();

	// by now we should have 1 remaining entity in the original IntsArchetype (no point in testing it),
	// and 3 entities in the Ints&Floats archetype, with value of Int fragment and Float fragment matching.
	for (int32 EntityIndex = 0; EntityIndex < EntitiesCount; ++EntityIndex)
	{
		if (EntityIndex != MiddleIndex)
		{
			const FTestFragment_Float& FloatFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(EntitiesCreated[EntityIndex]);
			const FTestFragment_Int& IntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(EntitiesCreated[EntityIndex]);

			INFO("All int fragments are expected to contain the value indicating the order in which the entity has been created.");
			CHECK(IntFragment.Value == EntityIndex);
			INFO("The int and float values are expected to remain in sync.");
			CHECK(IntFragment.Value == static_cast<int32>(FloatFragment.Value));
		}
	}
}

//-----------------------------------------------------------------------------
// Latent tests (Phase 3) — entity creation in scheduled processors
// These use FMassLLTProcessingPhasesFixture to drive multi-frame processing.
//-----------------------------------------------------------------------------

DEFINE_LOG_CATEGORY_STATIC(LogMassComplexTest, Log, All);

//-----------------------------------------------------------------------------
// Helper: creates 3 processors that each spawn NumToSpawn entities into
// Ints/Floats/Large archetypes via BuildEntity. Used by CreateEntitiesParallel
// and BatchedCreateEntitiesParallel variants.
//-----------------------------------------------------------------------------
static void SetupParallelCreationProcessors(
	FMassLLTProcessingPhasesFixture& F,
	int32 NumToSpawn,
	bool bUseBatchReserve,
	TObjectPtr<UMassLLTProcessorBase>& OutCreateInts,
	TObjectPtr<UMassLLTProcessorBase>& OutCreateFloats,
	TObjectPtr<UMassLLTProcessorBase>& OutCreateLarge)
{
	constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	const FMassElementBitSet IntsComposition = F.IntsArchetype.GetCompositionBitSetChecked();
	OutCreateInts = NewObject<UMassLLTProcessorBase>();
	OutCreateInts->SetShouldAllowMultipleInstances(true);
	OutCreateInts->SetProcessingPhase(PhaseToRun);
	OutCreateInts->GetEntityCreationRequirements().AddCreatedArchetype(IntsComposition);
	OutCreateInts->ExecutionFunction = [&F, NumToSpawn, bUseBatchReserve](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			UE_LOGF(LogMassComplexTest, Log, "Begin Creating IntsArchetype");

			TArray<FMassEntityHandle> Entities;
			F.EntityManager->BatchReserveEntities(NumToSpawn, Entities);
			for (int32 EntityIndex = 0; EntityIndex < NumToSpawn; ++EntityIndex)
			{
				Context.BuildEntity(F.IntsArchetype, Entities[EntityIndex]);
				if (!bUseBatchReserve)
				{
					FMassEntityView EntityView(ExecuteEntityManager, Entities[EntityIndex]);
					EntityView.GetFragmentData<FTestFragment_Int>().Value = 1;
				}
			}

			UE_LOGF(LogMassComplexTest, Log, "End Creating IntsArchetype");
		};
	F.PhaseManager->RegisterDynamicProcessor(*OutCreateInts);

	const FMassElementBitSet FloatsComposition = F.FloatsArchetype.GetCompositionBitSetChecked();
	OutCreateFloats = NewObject<UMassLLTProcessorBase>();
	OutCreateFloats->SetShouldAllowMultipleInstances(true);
	OutCreateFloats->SetProcessingPhase(PhaseToRun);
	OutCreateFloats->GetEntityCreationRequirements().AddCreatedArchetype(FloatsComposition);
	OutCreateFloats->ExecutionFunction = [&F, NumToSpawn, bUseBatchReserve](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			UE_LOGF(LogMassComplexTest, Log, "Begin Creating FloatsArchetype");

			TArray<FMassEntityHandle> Entities;
			F.EntityManager->BatchReserveEntities(NumToSpawn, Entities);
			for (int32 EntityIndex = 0; EntityIndex < NumToSpawn; ++EntityIndex)
			{
				Context.BuildEntity(F.FloatsArchetype, Entities[EntityIndex]);
				if (!bUseBatchReserve)
				{
					FMassEntityView EntityView(ExecuteEntityManager, Entities[EntityIndex]);
					EntityView.GetFragmentData<FTestFragment_Float>().Value = 1.0f;
				}
			}

			UE_LOGF(LogMassComplexTest, Log, "End Creating FloatsArchetype");
		};
	F.PhaseManager->RegisterDynamicProcessor(*OutCreateFloats);

	const FMassElementBitSet LargeComposition = F.LargeArchetype.GetCompositionBitSetChecked();
	OutCreateLarge = NewObject<UMassLLTProcessorBase>();
	OutCreateLarge->SetShouldAllowMultipleInstances(true);
	OutCreateLarge->SetProcessingPhase(PhaseToRun);
	OutCreateLarge->GetEntityCreationRequirements().AddCreatedArchetype(LargeComposition);
	OutCreateLarge->ExecutionFunction = [&F, NumToSpawn](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			UE_LOGF(LogMassComplexTest, Log, "Begin Creating LargeArchetype");

			TArray<FMassEntityHandle> Entities;
			F.EntityManager->BatchReserveEntities(NumToSpawn, Entities);
			for (int32 EntityIndex = 0; EntityIndex < NumToSpawn; ++EntityIndex)
			{
				Context.BuildEntity(F.LargeArchetype, Entities[EntityIndex]);
			}

			UE_LOGF(LogMassComplexTest, Log, "End Creating LargeArchetype");
		};
	F.PhaseManager->RegisterDynamicProcessor(*OutCreateLarge);
}

static void CleanupParallelCreationProcessors(
	FMassLLTProcessingPhasesFixture& F,
	TObjectPtr<UMassLLTProcessorBase>& CreateInts,
	TObjectPtr<UMassLLTProcessorBase>& CreateFloats,
	TObjectPtr<UMassLLTProcessorBase>& CreateLarge)
{
	F.PhaseManager->UnregisterDynamicProcessor(*CreateInts);
	CreateInts->MarkAsGarbage();
	CreateInts = nullptr;
	F.PhaseManager->UnregisterDynamicProcessor(*CreateFloats);
	CreateFloats->MarkAsGarbage();
	CreateFloats = nullptr;
	F.PhaseManager->UnregisterDynamicProcessor(*CreateLarge);
	CreateLarge->MarkAsGarbage();
	CreateLarge = nullptr;
}

//-----------------------------------------------------------------------------
// CreateEntitiesParallel — 3 processors each create 1M entities via BuildEntity
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::Complex::CreateEntitiesParallel", "[Mass][Complex]")
{
	static constexpr int32 NumToSpawn = 1000000;

	InitializePhaseManager();

	TObjectPtr<UMassLLTProcessorBase> CreateInts;
	TObjectPtr<UMassLLTProcessorBase> CreateFloats;
	TObjectPtr<UMassLLTProcessorBase> CreateLarge;
	SetupParallelCreationProcessors(*this, NumToSpawn, /*bUseBatchReserve=*/false, CreateInts, CreateFloats, CreateLarge);

	// Single tick
	Tick();

	WaitForCompletion();

#if WITH_MASSENTITY_DEBUG
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype) == NumToSpawn);
#endif

	CleanupParallelCreationProcessors(*this, CreateInts, CreateFloats, CreateLarge);
}

//-----------------------------------------------------------------------------
// CreateEntitiesParallelWithProcessingQueue — same as above (queue variant)
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::Complex::CreateEntitiesParallelWithProcessingQueue", "[Mass][Complex]")
{
	static constexpr int32 NumToSpawn = 1000000;

	InitializePhaseManager();

	TObjectPtr<UMassLLTProcessorBase> CreateInts;
	TObjectPtr<UMassLLTProcessorBase> CreateFloats;
	TObjectPtr<UMassLLTProcessorBase> CreateLarge;
	SetupParallelCreationProcessors(*this, NumToSpawn, /*bUseBatchReserve=*/false, CreateInts, CreateFloats, CreateLarge);

	Tick();

	WaitForCompletion();

#if WITH_MASSENTITY_DEBUG
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype) == NumToSpawn);
#endif

	CleanupParallelCreationProcessors(*this, CreateInts, CreateFloats, CreateLarge);
}

//-----------------------------------------------------------------------------
// CreateEntitiesDuringQuery — create entities during ForEachEntityChunk iteration
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::Complex::CreateEntitiesDuringQuery", "[Mass][Complex]")
{
	static constexpr int32 NumToSpawn = 10;
	constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	InitializePhaseManager();

	int32 NumEntitiesObserved = 0;

	// Set up an observer for entity creation
	UMassLLTProcessorBase* TestObserver = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	TestObserver->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::CreateEntity, TestObserver);
	TestObserver->ExecutionFunction = [TestObserver, &NumEntitiesObserved](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			TestObserver->EntityQuery.ForEachEntityChunk(Context, [&NumEntitiesObserved](FMassExecutionContext& Context)
				{
					for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						NumEntitiesObserved++;
					}
				});
		};

	// Create initial entities in IntsArchetype
	TArray<FMassEntityHandle> Created;
	Created.Reserve(NumToSpawn);
	EntityManager->BatchCreateEntities(IntsArchetype, NumToSpawn, Created);

	const FMassElementBitSet FloatsComposition = FloatsArchetype.GetCompositionBitSetChecked();
	const FMassElementBitSet LargeComposition = LargeArchetype.GetCompositionBitSetChecked();

	TObjectPtr<UMassLLTProcessorBase> CreateEntitiesProcessor = NewObject<UMassLLTProcessorBase>();
	CreateEntitiesProcessor->SetShouldAllowMultipleInstances(true);
	CreateEntitiesProcessor->SetProcessingPhase(PhaseToRun);
	CreateEntitiesProcessor->CallInitialize(GetTransientPackageAsObject(), EntityManager.ToSharedRef());
	CreateEntitiesProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
	CreateEntitiesProcessor->EntityQuery.AddCreatedArchetype(FloatsComposition);
	CreateEntitiesProcessor->EntityQuery.AddCreatedArchetype(LargeComposition);

	CreateEntitiesProcessor->ExecutionFunction = [&CreateEntitiesProcessor, this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			CreateEntitiesProcessor->EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
				{
					for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						Context.CreateEntity(FloatsArchetype);
						Context.CreateEntity(LargeArchetype);
					}
				});
		};
	PhaseManager->RegisterDynamicProcessor(*CreateEntitiesProcessor);

	Tick();

	WaitForCompletion();

#if WITH_MASSENTITY_DEBUG
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == NumToSpawn);
	INFO("Expected number of observed entities to match number created");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == NumEntitiesObserved);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype) == NumToSpawn);
#endif

	PhaseManager->UnregisterDynamicProcessor(*CreateEntitiesProcessor);
	CreateEntitiesProcessor->MarkAsGarbage();
	CreateEntitiesProcessor = nullptr;
}

//-----------------------------------------------------------------------------
// DeferredCreateEntitiesDuringQuery — deferred entity creation during iteration
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::Complex::DeferredCreateEntitiesDuringQuery", "[Mass][Complex]")
{
	static constexpr int32 NumToSpawn = 50000;
	constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	InitializePhaseManager();

	TArray<FMassEntityHandle> Created;
	Created.Reserve(NumToSpawn);
	EntityManager->BatchCreateEntities(IntsArchetype, NumToSpawn, Created);

	TObjectPtr<UMassLLTProcessorBase> CreateEntitiesProcessor = NewObject<UMassLLTProcessorBase>();
	CreateEntitiesProcessor->SetShouldAllowMultipleInstances(true);
	CreateEntitiesProcessor->SetProcessingPhase(PhaseToRun);
	CreateEntitiesProcessor->CallInitialize(GetTransientPackageAsObject(), EntityManager.ToSharedRef());
	CreateEntitiesProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);

	CreateEntitiesProcessor->ExecutionFunction = [&CreateEntitiesProcessor, this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			CreateEntitiesProcessor->EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
				{
					for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						Context.Defer().PushCommand<FMassDeferredCreateCommand>(
							[this](FMassEntityManager& DeferredEntityManager)
							{
								DeferredEntityManager.CreateEntity(FloatsArchetype);
							}
						);

						Context.Defer().PushCommand<FMassDeferredCreateCommand>(
							[this](FMassEntityManager& DeferredEntityManager)
							{
								DeferredEntityManager.CreateEntity(LargeArchetype);
							}
						);
					}
				});
		};
	PhaseManager->RegisterDynamicProcessor(*CreateEntitiesProcessor);

	Tick();

	WaitForCompletion();

#if WITH_MASSENTITY_DEBUG
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype) == NumToSpawn);
#endif

	PhaseManager->UnregisterDynamicProcessor(*CreateEntitiesProcessor);
	CreateEntitiesProcessor->MarkAsGarbage();
	CreateEntitiesProcessor = nullptr;
}

//-----------------------------------------------------------------------------
// BatchedCreateEntitiesParallel — batched variant of parallel entity creation
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::Complex::BatchedCreateEntitiesParallel", "[Mass][Complex]")
{
	static constexpr int32 NumToSpawn = 1000000;

	InitializePhaseManager();

	TObjectPtr<UMassLLTProcessorBase> CreateInts;
	TObjectPtr<UMassLLTProcessorBase> CreateFloats;
	TObjectPtr<UMassLLTProcessorBase> CreateLarge;
	SetupParallelCreationProcessors(*this, NumToSpawn, /*bUseBatchReserve=*/true, CreateInts, CreateFloats, CreateLarge);

	Tick();

	WaitForCompletion();

#if WITH_MASSENTITY_DEBUG
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype) == NumToSpawn);
#endif

	CleanupParallelCreationProcessors(*this, CreateInts, CreateFloats, CreateLarge);
}

//-----------------------------------------------------------------------------
// CreateEntitiesDeferred — deferred entity creation in processors
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::Complex::CreateEntitiesDeferred", "[Mass][Complex]")
{
	static constexpr int32 NumToSpawn = 1000000;
	constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	InitializePhaseManager();

	TObjectPtr<UMassLLTProcessorBase> CreateIntsProcessor = NewObject<UMassLLTProcessorBase>();
	CreateIntsProcessor->SetShouldAllowMultipleInstances(true);
	CreateIntsProcessor->SetProcessingPhase(PhaseToRun);
	CreateIntsProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			Context.Defer().PushCommand<FMassDeferredCreateCommand>(
				[this](FMassEntityManager& DeferredEntityManager)
				{
					for (int32 EntityIndex = 0; EntityIndex < NumToSpawn; ++EntityIndex)
					{
						DeferredEntityManager.CreateEntity(IntsArchetype);
					}
				}
			);
		};
	PhaseManager->RegisterDynamicProcessor(*CreateIntsProcessor);

	TObjectPtr<UMassLLTProcessorBase> CreateFloatsProcessor = NewObject<UMassLLTProcessorBase>();
	CreateFloatsProcessor->SetShouldAllowMultipleInstances(true);
	CreateFloatsProcessor->SetProcessingPhase(PhaseToRun);
	CreateFloatsProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			Context.Defer().PushCommand<FMassDeferredCreateCommand>(
				[this](FMassEntityManager& DeferredEntityManager)
				{
					for (int32 EntityIndex = 0; EntityIndex < NumToSpawn; ++EntityIndex)
					{
						DeferredEntityManager.CreateEntity(FloatsArchetype);
					}
				}
			);
		};
	PhaseManager->RegisterDynamicProcessor(*CreateFloatsProcessor);

	TObjectPtr<UMassLLTProcessorBase> CreateLargeProcessor = NewObject<UMassLLTProcessorBase>();
	CreateLargeProcessor->SetShouldAllowMultipleInstances(true);
	CreateLargeProcessor->SetProcessingPhase(PhaseToRun);
	CreateLargeProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			for (int32 EntityIndex = 0; EntityIndex < NumToSpawn; ++EntityIndex)
			{
				Context.Defer().PushCommand<FMassDeferredCreateCommand>(
					[this](FMassEntityManager& DeferredEntityManager)
					{
						DeferredEntityManager.CreateEntity(LargeArchetype);
					}
				);
			}
		};
	PhaseManager->RegisterDynamicProcessor(*CreateLargeProcessor);

	Tick();

	WaitForCompletion();

#if WITH_MASSENTITY_DEBUG
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == NumToSpawn);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype) == NumToSpawn);
#endif

	PhaseManager->UnregisterDynamicProcessor(*CreateIntsProcessor);
	CreateIntsProcessor->MarkAsGarbage();
	CreateIntsProcessor = nullptr;
	PhaseManager->UnregisterDynamicProcessor(*CreateFloatsProcessor);
	CreateFloatsProcessor->MarkAsGarbage();
	CreateFloatsProcessor = nullptr;
	PhaseManager->UnregisterDynamicProcessor(*CreateLargeProcessor);
	CreateLargeProcessor->MarkAsGarbage();
	CreateLargeProcessor = nullptr;
}

//-----------------------------------------------------------------------------
// DestroyEntitiesDuringQuery — create entities, then destroy them during query
// iteration using deferred commands with mass.FullyParallel=false
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::Complex::DestroyEntitiesDuringQuery", "[Mass][Complex]")
{
	static constexpr int32 NumToSpawn = 10;
	constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	// Force Mass to run synchronously via CVar
	bool bOldFullyParallel = true;
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("mass.FullyParallel")))
	{
		bOldFullyParallel = CVar->GetBool();
		CVar->Set(false, ECVF_SetByCode);
	}

	InitializePhaseManager();

	// Spawn entities
	TArray<FMassEntityHandle> Created;
	Created.Reserve(NumToSpawn);
	EntityManager->BatchCreateEntities(IntsArchetype, NumToSpawn, Created);

	// Setup the processor
	TObjectPtr<UMassLLTProcessorBase> DestroyEntitiesProcessor = NewObject<UMassLLTProcessorBase>();
	DestroyEntitiesProcessor->SetShouldAllowMultipleInstances(true);
	DestroyEntitiesProcessor->SetProcessingPhase(PhaseToRun);
	DestroyEntitiesProcessor->CallInitialize(GetTransientPackageAsObject(), EntityManager.ToSharedRef());
	DestroyEntitiesProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);

	DestroyEntitiesProcessor->ExecutionFunction = [&DestroyEntitiesProcessor](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
		{
			DestroyEntitiesProcessor->EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
				{
					for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						FMassEntityHandle Entity = Context.GetEntity(EntityIt);
						Context.Defer().DestroyEntity(Entity);
					}
				});
		};
	PhaseManager->RegisterDynamicProcessor(*DestroyEntitiesProcessor);

	Tick();

	WaitForCompletion();

#if WITH_MASSENTITY_DEBUG
	INFO("Expected IntsArchetype count to be 0 after synchronous destruction");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 0);
#endif

	PhaseManager->UnregisterDynamicProcessor(*DestroyEntitiesProcessor);
	DestroyEntitiesProcessor->MarkAsGarbage();
	DestroyEntitiesProcessor = nullptr;

	// Restore the CVar
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("mass.FullyParallel")))
	{
		CVar->Set(bOldFullyParallel, ECVF_SetByCode);
	}
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
