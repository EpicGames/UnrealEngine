// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "MassExecutor.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Processor::Composite::Empty", "[Mass][Processor][Composite]")
{
	REQUIRE(EntityManager);

	UMassCompositeProcessor* CompositeProcessor = NewObject<UMassCompositeProcessor>();
	REQUIRE(CompositeProcessor != nullptr);
	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
	// it should just run, no warnings
	UE::Mass::Executor::Run(*CompositeProcessor, ProcessingContext);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Processor::Composite::NoWork", "[Mass][Processor][Composite]")
{
	REQUIRE(EntityManager);

	UMassCompositeProcessor* CompositeProcessor = NewObject<UMassCompositeProcessor>();
	REQUIRE(CompositeProcessor != nullptr);

	int32 TimesExecuted = 0;
	FMassExecuteFunction QueryExecFunction = [&TimesExecuted](FMassExecutionContext& Context)
	{
		++TimesExecuted;
	};

	{
		TArray<UMassProcessor*> Processors;
		for (int32 ProcessorIndex = 0; ProcessorIndex < 3; ++ProcessorIndex)
		{
			UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
			Processor->ForEachEntityChunkExecutionFunction = QueryExecFunction;
			// need to set up some requirements to make EntityQuery valid
			Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
			Processors.Add(Processor);
		}

		CompositeProcessor->SetChildProcessors(Processors);
	}

	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
	UE::Mass::Executor::Run(*CompositeProcessor, ProcessingContext);
	INFO("None of the execution functions should have been executed");
	CHECK(TimesExecuted == 0);

	// now test there being some entities but of different composition. We create Float entities, but processors
	// require Ints.
	constexpr int32 NumberOfEntitieToCreate = 17;
	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumberOfEntitieToCreate, EntitiesCreated);

	TimesExecuted = 0;
	{
		TArray<TObjectPtr<UMassProcessor>> Processors;
		for (int32 ProcessorIndex = 0; ProcessorIndex < 3; ++ProcessorIndex)
		{
			UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
			Processor->ForEachEntityChunkExecutionFunction = QueryExecFunction;
			Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
			Processors.Add(Processor);
		}

		CompositeProcessor->SetChildProcessors(MoveTemp(Processors));
	}

	UE::Mass::Executor::Run(*CompositeProcessor, ProcessingContext);
	INFO("None of the execution functions should have been executed");
	CHECK(TimesExecuted == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Processor::Composite::MultipleSubProcessors", "[Mass][Processor][Composite]")
{
	REQUIRE(EntityManager);
	// we create a single entity so that the processors' execution functions would get called.
	EntityManager->CreateEntity(IntsArchetype);

	UMassCompositeProcessor* CompositeProcessor = NewObject<UMassCompositeProcessor>();
	REQUIRE(CompositeProcessor != nullptr);

	int32 ExpectedResult = 0;
	int32 Result = 0;
	{
		TArray<UMassProcessor*> Processors;
		for (int32 ProcessorIndex = 0; ProcessorIndex < 3; ++ProcessorIndex)
		{
			UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
			Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
			int32 Value = static_cast<int32>(FMath::Pow(10.f, static_cast<float>(ProcessorIndex)));
			Processor->ForEachEntityChunkExecutionFunction = [&Result, Value](FMassExecutionContext& Context)
			{
				Result += Value;
			};
			ExpectedResult += Value;

			Processors.Add(Processor);
		}

		CompositeProcessor->SetChildProcessors(Processors);
	}

	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
	UE::Mass::Executor::Run(*CompositeProcessor, ProcessingContext);
	INFO("All of the child processors should get run");
	CHECK(Result == ExpectedResult);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
