// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassArchetypeTypes.h"
#include "MassEntityTypes.h"
#include "MassExecutionContext.h"
#include "MassEntityQuery.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Archetype.VersionedHandleInvalidation", "[Mass][Coverage][Archetype]")
{
	REQUIRE(EntityManager);

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);

	// Get a versioned handle
	FMassArchetypeVersionedHandle VersionedHandle(IntsArchetype);

	INFO("Versioned handle is valid initially");
	CHECK(VersionedHandle.IsValid());

	// Mutate entities — destroying should change entity order version
	EntityManager->DestroyEntity(Entities[0]);
	EntityManager->DestroyEntity(Entities[5]);

	// The versioned handle should now be out of date (entity order changed)
	INFO("Versioned handle is NOT up to date after entity destruction");
	CHECK_FALSE(VersionedHandle.IsUpToDate());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Archetype.SharedFragmentPerChunkStorage", "[Mass][Coverage][Archetype]")
{
	REQUIRE(EntityManager);

	// Create archetype with fragment + const shared fragment type
	FMassArchetypeCompositionDescriptor Composition;
	Composition.Add<FTestFragment_Int>();
	Composition.Add<FTestConstSharedFragment_Int>();
	const FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(Composition);

	// Create entities with DIFFERENT shared values — they should be in the SAME archetype
	// (shared fragment TYPES define archetype composition, VALUES are per-chunk)
	FMassArchetypeSharedFragmentValues SharedValuesA;
	SharedValuesA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(42));
	FMassArchetypeSharedFragmentValues SharedValuesB;
	SharedValuesB.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(99));

	TArray<FMassEntityHandle> EntitiesA;
	EntityManager->BatchCreateEntities(Archetype, SharedValuesA, 5, EntitiesA);
	TArray<FMassEntityHandle> EntitiesB;
	EntityManager->BatchCreateEntities(Archetype, SharedValuesB, 5, EntitiesB);

	// Different shared values -> same archetype (archetype is defined by types, not values)
	const FMassArchetypeHandle ArchA = EntityManager->GetArchetypeForEntity(EntitiesA[0]);
	const FMassArchetypeHandle ArchB = EntityManager->GetArchetypeForEntity(EntitiesB[0]);
	INFO("Different shared values produce SAME archetype");
	CHECK(ArchA == ArchB);

	// But queries observe distinct shared values per chunk
	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	Query.AddConstSharedRequirement<FTestConstSharedFragment_Int>();

	TSet<int32> ObservedSharedValues;
	FMassExecutionContext ExecutionContext(*EntityManager);
	Query.ForEachEntityChunk(ExecutionContext,
		[&ObservedSharedValues](FMassExecutionContext& Context)
	{
		const FTestConstSharedFragment_Int& SharedFrag = Context.GetConstSharedFragment<FTestConstSharedFragment_Int>();
		ObservedSharedValues.Add(SharedFrag.Value);
	});

	INFO("Query observed 2 distinct shared values across chunks");
	CHECK(ObservedSharedValues.Num() == 2);
	INFO("Observed value 42");
	CHECK(ObservedSharedValues.Contains(42));
	INFO("Observed value 99");
	CHECK(ObservedSharedValues.Contains(99));
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
