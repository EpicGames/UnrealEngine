// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassCommandBuffer.h"
#include "MassEntityView.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.CommandBuffer.CancelCommands", "[Mass][Coverage][CommandBuffer]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);

	// Queue commands
	EntityManager->Defer().AddFragment<FTestFragment_Int>(Entity);
	EntityManager->Defer().AddTag<FTestTag_A>(Entity);

	// Cancel all pending commands
	EntityManager->Defer().CancelCommands();

	// Flush — should be a no-op
	EntityManager->FlushCommands();

	// Entity should still be in FloatsArchetype with no Int fragment and no tag
	INFO("Entity still in original archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsArchetype);

	FMassEntityView View(*EntityManager, Entity);
	INFO("FTestFragment_Int should NOT be present");
	CHECK(View.GetFragmentDataPtr<FTestFragment_Int>() == nullptr);
	INFO("FTestTag_A should NOT be present");
	CHECK_FALSE(View.HasTag<FTestTag_A>());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.CommandBuffer.MoveAppend", "[Mass][Coverage][CommandBuffer]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(EmptyArchetype);

	FMassCommandBuffer SeparateBuffer;
	SeparateBuffer.AddFragment<FTestFragment_Int>(Entity);

	INFO("Separate buffer has pending commands");
	CHECK(SeparateBuffer.HasPendingCommands());

	// Add Float via the manager's deferred buffer
	EntityManager->Defer().AddFragment<FTestFragment_Float>(Entity);

	// MoveAppend the separate buffer into the manager's deferred buffer
	EntityManager->Defer().MoveAppend(SeparateBuffer);

	INFO("Separate buffer is empty after MoveAppend");
	CHECK_FALSE(SeparateBuffer.HasPendingCommands());
	INFO("Deferred buffer has pending commands after MoveAppend");
	CHECK(EntityManager->Defer().HasPendingCommands());

	// Flush via the entity manager — both commands should execute
	EntityManager->FlushCommands();

	FMassEntityView View(*EntityManager, Entity);
	INFO("Entity has FTestFragment_Float after flush");
	REQUIRE(View.GetFragmentDataPtr<FTestFragment_Float>() != nullptr);
	INFO("Entity has FTestFragment_Int after flush");
	REQUIRE(View.GetFragmentDataPtr<FTestFragment_Int>() != nullptr);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.CommandBuffer.HasPendingCommands", "[Mass][Coverage][CommandBuffer]")
{
	REQUIRE(EntityManager);

	INFO("No pending commands initially");
	CHECK_FALSE(EntityManager->Defer().HasPendingCommands());

	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
	EntityManager->Defer().AddTag<FTestTag_A>(Entity);

	INFO("Has pending commands after AddTag");
	CHECK(EntityManager->Defer().HasPendingCommands());

	EntityManager->FlushCommands();

	INFO("No pending commands after flush");
	CHECK_FALSE(EntityManager->Defer().HasPendingCommands());
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
