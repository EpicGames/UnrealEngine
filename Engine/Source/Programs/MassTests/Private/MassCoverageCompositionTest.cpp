// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityTypes.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Composition.Difference", "[Mass][Coverage][Composition]")
{
	// A = {FragFloat, FragInt, TagA}
	FMassArchetypeCompositionDescriptor A;
	A.Add<FTestFragment_Float>();
	A.Add<FTestFragment_Int>();
	A.Add<FTestTag_A>();

	// B = {FragInt, FragBool, TagA, TagB}
	FMassArchetypeCompositionDescriptor B;
	B.Add<FTestFragment_Int>();
	B.Add<FTestFragment_Bool>();
	B.Add<FTestTag_A>();
	B.Add<FTestTag_B>();

	// A.CalculateDifference(B) = elements in A not in B = {FragFloat}
	const FMassArchetypeCompositionDescriptor AMinusB = A.CalculateDifference(B);
	INFO("A-B contains FragFloat");
	CHECK(AMinusB.Contains<FTestFragment_Float>());
	INFO("A-B does not contain FragInt");
	CHECK_FALSE(AMinusB.Contains<FTestFragment_Int>());
	INFO("A-B does not contain TagA");
	CHECK_FALSE(AMinusB.Contains<FTestTag_A>());
	INFO("A-B does not contain FragBool");
	CHECK_FALSE(AMinusB.Contains<FTestFragment_Bool>());
	INFO("A-B has 1 stored type");
	CHECK(AMinusB.CountStoredTypes() == 1);

	// B.CalculateDifference(A) = elements in B not in A = {FragBool, TagB}
	const FMassArchetypeCompositionDescriptor BMinusA = B.CalculateDifference(A);
	INFO("B-A contains FragBool");
	CHECK(BMinusA.Contains<FTestFragment_Bool>());
	INFO("B-A contains TagB");
	CHECK(BMinusA.Contains<FTestTag_B>());
	INFO("B-A does not contain FragInt");
	CHECK_FALSE(BMinusA.Contains<FTestFragment_Int>());
	INFO("B-A has 2 stored types");
	CHECK(BMinusA.CountStoredTypes() == 2);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Composition.HasAll", "[Mass][Coverage][Composition]")
{
	FMassArchetypeCompositionDescriptor Full;
	Full.Add<FTestFragment_Float>();
	Full.Add<FTestFragment_Int>();
	Full.Add<FTestTag_A>();

	FMassArchetypeCompositionDescriptor Subset;
	Subset.Add<FTestFragment_Float>();
	Subset.Add<FTestTag_A>();

	FMassArchetypeCompositionDescriptor NonSubset;
	NonSubset.Add<FTestFragment_Float>();
	NonSubset.Add<FTestFragment_Bool>(); // Not in Full

	INFO("Full HasAll of valid subset");
	CHECK(Full.HasAll(Subset));
	INFO("Full does NOT HasAll of non-subset");
	CHECK_FALSE(Full.HasAll(NonSubset));

	// Empty descriptor is a subset of everything
	FMassArchetypeCompositionDescriptor Empty;
	INFO("Full HasAll of empty");
	CHECK(Full.HasAll(Empty));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Composition.CountStoredTypes", "[Mass][Coverage][Composition]")
{
	FMassArchetypeCompositionDescriptor Desc;
	INFO("Empty composition has 0 types");
	CHECK(Desc.CountStoredTypes() == 0);

	Desc.Add<FTestFragment_Float>();
	Desc.Add<FTestFragment_Int>();
	Desc.Add<FTestFragment_Bool>();
	Desc.Add<FTestTag_A>();
	Desc.Add<FTestTag_B>();
	Desc.Add<FTestSharedFragment_Int>();

	INFO("Composition with 3 fragments + 2 tags + 1 shared = 6 types");
	CHECK(Desc.CountStoredTypes() == 6);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
