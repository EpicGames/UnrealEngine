// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityView.h"
#include "MassEntityTypes.h"
#include "MassExecutionContext.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

#if WITH_MASSENTITY_DEBUG
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Manager.EntityCompaction", "[Mass][Coverage][Manager][Debug]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
	// Create enough entities to span at least 3 chunks
	const int32 TotalEntities = EntitiesPerChunk * 3;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, TotalEntities, Entities);

	// Set values so we can verify data integrity after compaction
	for (int32 i = 0; i < TotalEntities; ++i)
	{
		EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entities[i]).Value = i;
	}

	// Destroy every other entity to create fragmentation
	TArray<FMassEntityHandle> SurvivorEntities;
	for (int32 i = 0; i < TotalEntities; ++i)
	{
		if (i % 2 == 0)
		{
			EntityManager->DestroyEntity(Entities[i]);
		}
		else
		{
			SurvivorEntities.Add(Entities[i]);
		}
	}

	const int32 SurvivorCount = SurvivorEntities.Num();

	// Compact with unlimited time budget
	EntityManager->DoEntityCompaction(TNumericLimits<double>::Max());

	// Verify all surviving entities have intact data
	for (int32 i = 0; i < SurvivorCount; ++i)
	{
		INFO("Survivor entity is still valid");
		REQUIRE(EntityManager->IsEntityValid(SurvivorEntities[i]));
		const int32 OriginalIndex = i * 2 + 1; // Odd indices survived
		const int32 ActualValue = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(SurvivorEntities[i]).Value;
		INFO("Data integrity after compaction");
		CHECK(ActualValue == OriginalIndex);
	}
}
#endif // WITH_MASSENTITY_DEBUG

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Manager.SwapTagsForEntity", "[Mass][Coverage][Manager]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add TagA
	EntityManager->AddTagToEntity(Entity, FTestTag_A::StaticStruct());
	{
		FMassEntityView View(*EntityManager, Entity);
		INFO("Entity has TagA before swap");
		CHECK(View.HasTag<FTestTag_A>());
		INFO("Entity does not have TagB before swap");
		CHECK_FALSE(View.HasTag<FTestTag_B>());
	}

	// Swap TagA -> TagB
	EntityManager->SwapTagsForEntity(Entity, FTestTag_A::StaticStruct(), FTestTag_B::StaticStruct());

	{
		FMassEntityView View(*EntityManager, Entity);
		INFO("Entity does not have TagA after swap");
		CHECK_FALSE(View.HasTag<FTestTag_A>());
		INFO("Entity has TagB after swap");
		CHECK(View.HasTag<FTestTag_B>());
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Manager.ArchetypeTraversal", "[Mass][Coverage][Manager]")
{
	REQUIRE(EntityManager);

	// Create archetypes
	EntityManager->CreateEntity(FloatsArchetype);
	EntityManager->CreateEntity(IntsArchetype);
	EntityManager->CreateEntity(FloatsIntsArchetype);

	// Traverse FloatsIntsArchetype and collect fragment types
	TArray<const UScriptStruct*> CollectedTypes;
	FMassEntityManager::ForEachArchetypeFragmentType(FloatsIntsArchetype, [&CollectedTypes](const UScriptStruct* FragmentType)
	{
		CollectedTypes.Add(FragmentType);
	});

	INFO("FloatsInts archetype has 2 fragment types");
	CHECK(CollectedTypes.Num() == 2);

	const bool bHasFloat = CollectedTypes.Contains(FTestFragment_Float::StaticStruct());
	const bool bHasInt = CollectedTypes.Contains(FTestFragment_Int::StaticStruct());
	INFO("Contains FTestFragment_Float");
	CHECK(bHasFloat);
	INFO("Contains FTestFragment_Int");
	CHECK(bHasInt);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Manager.BatchAddElements", "[Mass][Coverage][Manager]")
{
	REQUIRE(EntityManager);

	const int32 NumEntities = 100;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

	// Verify all start without Int fragment
	for (const FMassEntityHandle& Entity : Entities)
	{
		CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsArchetype);
	}

	// Batch add FTestFragment_Int to all entities
	EntityManager->BatchAddElementToEntities(Entities, FTestFragment_Int::StaticStruct());

	// All entities should now be in FloatsInts archetype
	for (const FMassEntityHandle& Entity : Entities)
	{
		CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsIntsArchetype);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Manager.BatchRemoveElements", "[Mass][Coverage][Manager]")
{
	REQUIRE(EntityManager);

	const int32 NumEntities = 100;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumEntities, Entities);

	// Batch remove FTestFragment_Int from all entities
	EntityManager->BatchRemoveElementFromEntities(Entities, FTestFragment_Int::StaticStruct());

	// All entities should now be in Floats-only archetype
	for (const FMassEntityHandle& Entity : Entities)
	{
		CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsArchetype);
	}
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
