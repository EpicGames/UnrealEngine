// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityView.h"
#include "MassEntityTypes.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.EntityView.TypeErasedFragmentAccess", "[Mass][Coverage][EntityView]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
	FMassEntityView View(*EntityManager, Entity);

	// Typed access
	FTestFragment_Int& TypedRef = View.GetFragmentData<FTestFragment_Int>();

	// Type-erased access
	FStructView StructView = View.GetFragmentDataStruct(FTestFragment_Int::StaticStruct());

	INFO("Type-erased view is valid");
	CHECK(StructView.IsValid());
	INFO("Type-erased pointer matches typed pointer");
	CHECK(StructView.GetMemory() == reinterpret_cast<const uint8*>(&TypedRef));

	// Verify we can read through the type-erased view
	FTestFragment_Int* ErasedPtr = StructView.GetPtr<FTestFragment_Int>();
	INFO("Can cast type-erased view back to typed");
	REQUIRE(ErasedPtr != nullptr);
	INFO("Values match");
	CHECK(ErasedPtr->Value == TypedRef.Value);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.EntityView.TypeErasedSharedAccess", "[Mass][Coverage][EntityView]")
{
	REQUIRE(EntityManager);

	// Build archetype with shared and const shared fragments
	FMassArchetypeCompositionDescriptor Composition;
	Composition.Add<FTestFragment_Int>();
	Composition.Add<FTestSharedFragment_Int>();
	Composition.Add<FTestConstSharedFragment_Int>();

	FMassArchetypeSharedFragmentValues SharedValues;
	const FSharedStruct SharedInt = FSharedStruct::Make<FTestSharedFragment_Int>(42);
	SharedValues.Add(SharedInt);
	const FConstSharedStruct ConstSharedInt = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(99);
	SharedValues.Add(ConstSharedInt);
	SharedValues.Sort();

	// CreateArchetype takes only composition — shared values are passed to BatchCreateEntities
	const FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(Composition);
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(Archetype, SharedValues, 1, Entities);
	const FMassEntityHandle Entity = Entities[0];
	FMassEntityView View(*EntityManager, Entity);

	// Test type-erased const shared access
	FConstStructView ConstSharedView = View.GetConstSharedFragmentDataStruct(FTestConstSharedFragment_Int::StaticStruct());
	INFO("Const shared type-erased view is valid");
	CHECK(ConstSharedView.IsValid());
	const FTestConstSharedFragment_Int* ConstData = ConstSharedView.GetPtr<FTestConstSharedFragment_Int>();
	INFO("Const shared data is accessible");
	REQUIRE(ConstData != nullptr);
	INFO("Const shared value matches");
	CHECK(ConstData->Value == 99);

	// Test type-erased mutable shared access
	FStructView SharedView = View.GetSharedFragmentDataStruct(FTestSharedFragment_Int::StaticStruct());
	INFO("Shared type-erased view is valid");
	CHECK(SharedView.IsValid());
	FTestSharedFragment_Int* SharedData = SharedView.GetPtr<FTestSharedFragment_Int>();
	INFO("Shared data is accessible");
	REQUIRE(SharedData != nullptr);
	INFO("Shared value matches");
	CHECK(SharedData->Value == 42);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
