// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassExecutionContext.h"
#include "MassEntityQuery.h"
#include "MassObserverProcessor.h"
#include "MassEntityView.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

// @todo SubsystemAccessChecked requires a live UWorld for UWorldSubsystem instances.
// LLT creates FMassEntityManager with GetTransientPackageAsObject() (not a UWorld),
// so FMassSubsystemAccess::FetchSubsystemInstance will hit check(World) and crash.
DISABLED_TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.ExecutionContext.SubsystemAccessChecked", "[Mass][Coverage][ExecutionContext]")
{
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.ExecutionContext.AuxData", "[Mass][Coverage][ExecutionContext]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Set up observer that checks AuxData
	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);

	bool bAuxDataValid = false;
	EMassObservedOperation ObservedOp = EMassObservedOperation::MAX;
	const UScriptStruct* ObservedType = nullptr;

	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bAuxDataValid, &ObservedOp, &ObservedType](FMassExecutionContext& Context)
	{
		const FInstancedStruct& AuxData = Context.GetAuxData();
		if (AuxData.IsValid())
		{
			const FMassObserverExecutionContext* ObserverContext = AuxData.GetPtr<FMassObserverExecutionContext>();
			if (ObserverContext)
			{
				bAuxDataValid = true;
				ObservedOp = ObserverContext->GetOperationType();
				ObservedType = ObserverContext->GetCurrentType();
			}
		}
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	// Trigger observer via deferred tag addition
	EntityManager->Defer().AddTag<FTestTag_A>(Entity);
	EntityManager->FlushCommands();

	INFO("AuxData contained valid FMassObserverExecutionContext");
	CHECK(bAuxDataValid);
	INFO("Observed operation was AddElement");
	CHECK(ObservedOp == EMassObservedOperation::AddElement);
	INFO("Observed type was FTestTag_A");
	CHECK(ObservedType == FTestTag_A::StaticStruct());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.ExecutionContext.MidExecutionFlush", "[Mass][Coverage][ExecutionContext]")
{
	REQUIRE(EntityManager);

	// Create entity with Float fragment only
	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

	bool bFlushCalled = false;
	bool bHasFragmentAfterFlush = false;
	FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(/*DeltaSeconds=*/0);
	Query.ForEachEntityChunk(ExecutionContext, [&bFlushCalled, &bHasFragmentAfterFlush, this](FMassExecutionContext& Context)
	{
		// Defer adding Int fragment to a different entity (not the one being iterated)
		// and flush mid-execution
		const FMassEntityHandle NewEntity = EntityManager->CreateEntity(EmptyArchetype);
		Context.Defer().AddFragment<FTestFragment_Int>(NewEntity);
		Context.FlushDeferred();
		bFlushCalled = true;

		// Check if the deferred command executed - new entity should now have FTestFragment_Int
		bHasFragmentAfterFlush = EntityManager->GetArchetypeForEntity(NewEntity).IsValid()
			&& FMassEntityView(*EntityManager, NewEntity).GetFragmentDataPtr<FTestFragment_Int>() != nullptr;
	});

	INFO("Deferred AddFragment executed after FlushDeferred");
	CHECK(bHasFragmentAfterFlush);
	INFO("FlushDeferred was called during iteration");
	CHECK(bFlushCalled);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
