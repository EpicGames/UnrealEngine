// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassExecutionContext.h"
#include "MassObserverProcessor.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Observer.BatchElementNotification", "[Mass][Coverage][Observer]")
{
	REQUIRE(EntityManager);

	// Create entities in an archetype without Float
	const int32 NumEntities = 50;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Set up observer for FTestFragment_Float addition
	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

	int32 ObservedEntityCount = 0;
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&ObservedEntityCount](FMassExecutionContext& Context)
	{
		ObservedEntityCount += Context.GetNumEntities();
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	// Batch add element to all entities
	EntityManager->BatchAddElementToEntities(Entities, FTestFragment_Float::StaticStruct());

	INFO("Observer processed all entities from batch add");
	CHECK(ObservedEntityCount == NumEntities);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Observer.FlushOrdering", "[Mass][Coverage][Observer]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Track observer firing order
	TArray<FString> FiringOrder;

	// Observer for TagA addition
	UMassLLTProcessorBase* AddObserver = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	AddObserver->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	AddObserver->EntityQuery.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
	AddObserver->ForEachEntityChunkExecutionFunction = [&FiringOrder](FMassExecutionContext& Context)
	{
		FiringOrder.Add(TEXT("Add"));
	};

	// Observer for TagA removal
	UMassLLTProcessorBase* RemoveObserver = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	RemoveObserver->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	RemoveObserver->ForEachEntityChunkExecutionFunction = [&FiringOrder](FMassExecutionContext& Context)
	{
		FiringOrder.Add(TEXT("Remove"));
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Add, AddObserver);
	ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Remove, RemoveObserver);

	// Queue add then remove in same flush
	EntityManager->Defer().AddTag<FTestTag_A>(Entity);
	EntityManager->Defer().RemoveTag<FTestTag_A>(Entity);
	EntityManager->FlushCommands();

	// Both observers should have fired in order: add then remove
	INFO("Both observers fired");
	REQUIRE(FiringOrder.Num() == 2);
	INFO("Add observer fired first");
	CHECK(FiringOrder[0] == TEXT("Add"));
	INFO("Remove observer fired second");
	CHECK(FiringOrder[1] == TEXT("Remove"));
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
