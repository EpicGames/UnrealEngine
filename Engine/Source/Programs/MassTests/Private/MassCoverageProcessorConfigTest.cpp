// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.ProcessorConfig.DynamicMarking", "[Mass][Coverage][ProcessorConfig]")
{
	REQUIRE(EntityManager);

	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	INFO("Processor is not dynamic by default");
	CHECK_FALSE(Processor->IsDynamic());

	Processor->MarkAsDynamic();
	INFO("Processor is dynamic after MarkAsDynamic");
	CHECK(Processor->IsDynamic());

	// Verify dynamic processor still executes
	EntityManager->CreateEntity(FloatsArchetype);
	Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

	int32 ExecutionCount = 0;
	Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
	{
		++ExecutionCount;
	};

	FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
	UE::Mass::Executor::Run(*Processor, ProcessingContext);

	INFO("Dynamic processor executed successfully");
	CHECK(ExecutionCount == 1);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.ProcessorConfig.QueryBasedPruning", "[Mass][Coverage][ProcessorConfig]")
{
	REQUIRE(EntityManager);

	// Create entities with Float fragment only
	EntityManager->CreateEntity(FloatsArchetype);

	// Create processor that requires Bool fragment (not present in any entity)
	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	Processor->EntityQuery.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::ReadOnly);

	int32 ExecutionCount = 0;
	Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
	{
		++ExecutionCount;
	};

	// Execute — processor should not execute because no entities match
	FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
	UE::Mass::Executor::Run(*Processor, ProcessingContext);

	INFO("Processor with no matching entities did not execute");
	CHECK(ExecutionCount == 0);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
