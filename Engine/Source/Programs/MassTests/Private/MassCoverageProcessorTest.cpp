// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Processor.InitialState", "[Mass][Coverage][Processor]")
{
	REQUIRE(EntityManager);

	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	INFO("Newly created processor is active by default");
	CHECK(Processor->IsActive());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Processor.ActiveInactive", "[Mass][Coverage][Processor]")
{
	REQUIRE(EntityManager);

	EntityManager->CreateEntity(FloatsArchetype);

	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

	int32 ExecutionCount = 0;
	Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
	{
		++ExecutionCount;
	};

	// Make inactive and execute
	Processor->MakeInactive();
	INFO("Processor is not active after MakeInactive");
	CHECK_FALSE(Processor->IsActive());

	{
		FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
		UE::Mass::Executor::Run(*Processor, ProcessingContext);
	}
	INFO("Inactive processor did not execute");
	CHECK(ExecutionCount == 0);

	// Make active and execute
	Processor->MakeActive();
	INFO("Processor is active after MakeActive");
	CHECK(Processor->IsActive());

	{
		FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
		UE::Mass::Executor::Run(*Processor, ProcessingContext);
	}
	INFO("Active processor executed once");
	CHECK(ExecutionCount == 1);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Processor.OneShot", "[Mass][Coverage][Processor]")
{
	REQUIRE(EntityManager);

	EntityManager->CreateEntity(FloatsArchetype);

	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

	int32 ExecutionCount = 0;
	Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
	{
		++ExecutionCount;
	};

	Processor->MakeOneShot();
	INFO("OneShot processor is active before first execution");
	CHECK(Processor->IsActive());

	// First execution — should run
	{
		FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
		UE::Mass::Executor::Run(*Processor, ProcessingContext);
	}
	INFO("OneShot processor executed on first run");
	CHECK(ExecutionCount == 1);
	INFO("OneShot processor is inactive after first execution");
	CHECK_FALSE(Processor->IsActive());

	// Second execution — should NOT run
	{
		FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
		UE::Mass::Executor::Run(*Processor, ProcessingContext);
	}
	INFO("OneShot processor did not execute on second run");
	CHECK(ExecutionCount == 1);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Processor.OneShotReactivation", "[Mass][Coverage][Processor]")
{
	REQUIRE(EntityManager);

	EntityManager->CreateEntity(FloatsArchetype);

	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

	int32 ExecutionCount = 0;
	Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
	{
		++ExecutionCount;
	};

	// OneShot, execute, auto-deactivate
	Processor->MakeOneShot();
	{
		FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
		UE::Mass::Executor::Run(*Processor, ProcessingContext);
	}
	INFO("Executed once");
	CHECK(ExecutionCount == 1);
	INFO("Inactive after oneshot");
	CHECK_FALSE(Processor->IsActive());

	// Reactivate
	Processor->MakeActive();
	INFO("Active after reactivation");
	CHECK(Processor->IsActive());

	{
		FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
		UE::Mass::Executor::Run(*Processor, ProcessingContext);
	}
	INFO("Executed again after reactivation");
	CHECK(ExecutionCount == 2);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
