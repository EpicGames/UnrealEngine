// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityBuilder.h"
#include "MassRelationManager.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

using namespace UE::Mass;
using namespace UE::Mass::Relations;

//-----------------------------------------------------------------------------
// Custom fixture for relation coverage tests.
// Uses CleanUp for BOTH roles (no exclusivity, no hierarchy) — different from
// FMassLLTRelationFixture which uses Destroy+exclusive+hierarchical.
//-----------------------------------------------------------------------------
struct FMassLLTRelationCoverageFixture : FMassLLTEntityFixture
{
	UScriptStruct* RelationTypeA = nullptr;
	UScriptStruct* RelationTypeB = nullptr;
	FTypeHandle RelationHandleA;
	FTypeHandle RelationHandleB;
	FRelationManager* RelationManagerPtr = nullptr;

	static TNotNull<UScriptStruct*> MakeRelationType(FName TypeName)
	{
		UScriptStruct* NewStruct = NewObject<UScriptStruct>(UScriptStruct::StaticClass(),
			FName(*FString::Printf(TEXT("CoverageRelation_%s"), *TypeName.ToString())), RF_Public);
		NewStruct->SetSuperStruct(FMassRelation::StaticStruct());
		return NewStruct;
	}

	static FRelationTypeTraits CreateRelationTraits(TNotNull<UScriptStruct*> RelationType)
	{
		FRelationTypeTraits Traits(RelationType);
		Traits.RoleTraits[static_cast<uint8>(ERelationRole::Object)].DestructionPolicy = ERemovalPolicy::CleanUp;
		Traits.RoleTraits[static_cast<uint8>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::CleanUp;
		return Traits;
	}

	FMassLLTRelationCoverageFixture()
	{
		RelationTypeA = MakeRelationType(TEXT("TypeA"));
		RelationHandleA = EntityManager->GetTypeManager().RegisterType(CreateRelationTraits(RelationTypeA));

		RelationTypeB = MakeRelationType(TEXT("TypeB"));
		RelationHandleB = EntityManager->GetTypeManager().RegisterType(CreateRelationTraits(RelationTypeB));

		RelationManagerPtr = &EntityManager->GetRelationManager();

		REQUIRE(RelationHandleA.IsValid());
		REQUIRE(RelationHandleB.IsValid());
		REQUIRE(RelationManagerPtr != nullptr);
	}
};

TEST_CASE_METHOD(FMassLLTRelationCoverageFixture, "Mass.Coverage.Relation.RoleEntities", "[Mass][Coverage][Relation]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(IntsArchetype);
	const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(FloatsArchetype);

	const FMassEntityHandle RelationEntity = RelationManagerPtr->CreateRelationInstance(RelationHandleA, SubjectEntity, ObjectEntity);
	EntityManager->FlushCommands();

	INFO("Relation entity created");
	CHECK(RelationEntity.IsValid());

	// Query subjects of the object
	TArray<FMassEntityHandle> Subjects = RelationManagerPtr->GetRelationSubjects(RelationHandleA, ObjectEntity);
	INFO("One subject found for ObjectEntity");
	REQUIRE(Subjects.Num() == 1);
	INFO("Subject is correct entity");
	CHECK(Subjects[0] == SubjectEntity);

	// Query objects of the subject
	TArray<FMassEntityHandle> Objects = RelationManagerPtr->GetRelationObjects(RelationHandleA, SubjectEntity);
	INFO("One object found for SubjectEntity");
	REQUIRE(Objects.Num() == 1);
	INFO("Object is correct entity");
	CHECK(Objects[0] == ObjectEntity);
}

TEST_CASE_METHOD(FMassLLTRelationCoverageFixture, "Mass.Coverage.Relation.MultiType", "[Mass][Coverage][Relation]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle EntityA = EntityManager->CreateEntity(IntsArchetype);
	const FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype);
	const FMassEntityHandle EntityC = EntityManager->CreateEntity(IntsArchetype);

	// Create two different relation types from EntityA
	RelationManagerPtr->CreateRelationInstance(RelationHandleA, EntityA, EntityB);
	RelationManagerPtr->CreateRelationInstance(RelationHandleB, EntityA, EntityC);
	EntityManager->FlushCommands();

	// Query each type independently
	TArray<FMassEntityHandle> ObjectsA = RelationManagerPtr->GetRelationObjects(RelationHandleA, EntityA);
	INFO("One TypeA relation object");
	REQUIRE(ObjectsA.Num() == 1);
	INFO("TypeA object is EntityB");
	CHECK(ObjectsA[0] == EntityB);

	TArray<FMassEntityHandle> ObjectsB = RelationManagerPtr->GetRelationObjects(RelationHandleB, EntityA);
	INFO("One TypeB relation object");
	REQUIRE(ObjectsB.Num() == 1);
	INFO("TypeB object is EntityC");
	CHECK(ObjectsB[0] == EntityC);
}

TEST_CASE_METHOD(FMassLLTRelationCoverageFixture, "Mass.Coverage.Relation.DataAccess", "[Mass][Coverage][Relation]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(IntsArchetype);
	const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(FloatsArchetype);

	RelationManagerPtr->CreateRelationInstance(RelationHandleA, SubjectEntity, ObjectEntity);
	EntityManager->FlushCommands();

	const bool bIsSubject = RelationManagerPtr->IsSubjectOfRelation(RelationHandleA, SubjectEntity, ObjectEntity);
	INFO("SubjectEntity is subject of relation to ObjectEntity");
	CHECK(bIsSubject);

	const bool bIsNotSubject = RelationManagerPtr->IsSubjectOfRelation(RelationHandleA, ObjectEntity, SubjectEntity);
	INFO("ObjectEntity is NOT subject of relation to SubjectEntity");
	CHECK_FALSE(bIsNotSubject);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
