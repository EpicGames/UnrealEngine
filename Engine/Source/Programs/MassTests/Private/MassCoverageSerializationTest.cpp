// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "Mass/EntityHandle.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.Serialization.EntityHandleRoundTrip", "[Mass][Coverage][Serialization]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Original = EntityManager->CreateEntity(IntsArchetype);
	INFO("Original handle is valid");
	CHECK(Original.IsSet());

	// Round-trip through AsNumber/FromNumber
	const uint64 Number = Original.AsNumber();
	const FMassEntityHandle Reconstructed = FMassEntityHandle::FromNumber(Number);

	INFO("Reconstructed handle equals original");
	CHECK(Reconstructed == Original);
	INFO("Reconstructed handle is valid");
	CHECK(EntityManager->IsEntityValid(Reconstructed));

	// Verify they reference the same entity data
	FTestFragment_Int& OriginalData = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Original);
	OriginalData.Value = 777;
	const FTestFragment_Int& ReconstructedData = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Reconstructed);
	INFO("Both handles reference the same entity data");
	CHECK(ReconstructedData.Value == 777);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
