// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityTypes.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.SharedFragmentValues.Replace", "[Mass][Coverage][SharedFragmentValues]")
{
	REQUIRE(EntityManager);

	FMassArchetypeSharedFragmentValues SharedValues;
	const FSharedStruct OriginalShared = FSharedStruct::Make<FTestSharedFragment_Int>(10);
	SharedValues.Add(OriginalShared);

	// Verify original value
	const FTestSharedFragment_Int* Data = SharedValues.GetSharedFragmentStruct(FTestSharedFragment_Int::StaticStruct()).GetPtr<FTestSharedFragment_Int>();
	INFO("Original shared data accessible");
	REQUIRE(Data != nullptr);
	INFO("Original value is 10");
	CHECK(Data->Value == 10);

	// Replace with new value
	const FSharedStruct ReplacementShared = FSharedStruct::Make<FTestSharedFragment_Int>(42);
	SharedValues.ReplaceSharedFragments(MakeArrayView(&ReplacementShared, 1));

	// Verify replaced value
	const FTestSharedFragment_Int* ReplacedData = SharedValues.GetSharedFragmentStruct(FTestSharedFragment_Int::StaticStruct()).GetPtr<FTestSharedFragment_Int>();
	INFO("Replaced shared data accessible");
	REQUIRE(ReplacedData != nullptr);
	INFO("Replaced value is 42");
	CHECK(ReplacedData->Value == 42);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.SharedFragmentValues.DoesMatchComposition", "[Mass][Coverage][SharedFragmentValues]")
{
	FMassArchetypeSharedFragmentValues SharedValues;
	const FSharedStruct SharedInt = FSharedStruct::Make<FTestSharedFragment_Int>(1);
	SharedValues.Add(SharedInt);
	const FConstSharedStruct ConstSharedInt = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(2);
	SharedValues.Add(ConstSharedInt);

	// Matching composition
	FMassArchetypeCompositionDescriptor MatchingDesc;
	MatchingDesc.Add<FTestSharedFragment_Int>();
	MatchingDesc.Add<FTestConstSharedFragment_Int>();

	INFO("Matches composition with same shared types");
	CHECK(SharedValues.DoesMatchComposition(MatchingDesc));

	// Non-matching: missing a type
	FMassArchetypeCompositionDescriptor PartialDesc;
	PartialDesc.Add<FTestSharedFragment_Int>();
	// Missing FTestConstSharedFragment_Int

	INFO("Does not match composition with missing type");
	CHECK_FALSE(SharedValues.DoesMatchComposition(PartialDesc));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Coverage.SharedFragmentValues.CreateCombined", "[Mass][Coverage][SharedFragmentValues]")
{
	// Create original shared values with one shared fragment
	FMassArchetypeSharedFragmentValues Original;
	const FConstSharedStruct ConstSharedInt = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(10);
	Original.Add(ConstSharedInt);

	// Modification adds a new shared fragment
	FMassArchetypeSharedFragmentValues Modification;
	const FConstSharedStruct ConstSharedFloat = FConstSharedStruct::Make<FTestConstSharedFragment_Float>(3.14f);
	Modification.Add(ConstSharedFloat);

	// New composition includes both original and new types
	FMassElementBitSet NewComposition;
	NewComposition.Add(FTestConstSharedFragment_Int::StaticStruct());
	NewComposition.Add(FTestConstSharedFragment_Float::StaticStruct());

	FMassArchetypeSharedFragmentValues Combined = FMassArchetypeSharedFragmentValues::CreateCombined(
		Original, NewComposition, &Modification);

	// Verify combined has both fragments
	const FTestConstSharedFragment_Int* IntData = Combined.GetConstSharedFragmentStruct(
		FTestConstSharedFragment_Int::StaticStruct()).GetPtr<FTestConstSharedFragment_Int>();
	INFO("Combined has original const shared int");
	REQUIRE(IntData != nullptr);
	INFO("Original value preserved");
	CHECK(IntData->Value == 10);

	const FTestConstSharedFragment_Float* FloatData = Combined.GetConstSharedFragmentStruct(
		FTestConstSharedFragment_Float::StaticStruct()).GetPtr<FTestConstSharedFragment_Float>();
	INFO("Combined has added const shared float");
	CHECK(FloatData != nullptr);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
