// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassObserverNotificationTypes.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::CreationContext::Append", "[Mass][CreationContext]")
{
	constexpr int32 IntEntitiesToSpawnCount = 6;
	constexpr int32 FloatEntitiesToSpawnCount = 7;

	TArray<FMassEntityHandle> Entities;
	TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContextInt = EntityManager->BatchCreateEntities(IntsArchetype, IntEntitiesToSpawnCount, Entities);
	TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContextFloat = EntityManager->BatchCreateEntities(FloatsArchetype, FloatEntitiesToSpawnCount, Entities);
	const int32 NumDifferentArchetypesUsed = 2;

	CHECK(CreationContextInt == CreationContextFloat);
	CHECK(CreationContextInt->DebugAreEntityCollectionsUpToDate());

	TArray<FMassArchetypeEntityCollection> EntityCollections = CreationContextInt->GetEntityCollections(*EntityManager.Get());
	CHECK(EntityCollections.Num() == NumDifferentArchetypesUsed);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::CreationContext::ManualCreate", "[Mass][CreationContext]")
{
	constexpr int32 IntEntitiesToSpawnCount = 6;
	int32 NumDifferentArchetypesUsed = 0;

	TArray<FMassEntityHandle> Entities;
	TSharedRef<FMassEntityManager::FEntityCreationContext> ObtainedContext = EntityManager->GetOrMakeCreationContext();
	{
		TSharedRef<FMassEntityManager::FEntityCreationContext> ObtainedContextCopy = EntityManager->GetOrMakeCreationContext();
		CHECK(ObtainedContext == ObtainedContextCopy);
	}

	{
		TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContextInt = EntityManager->BatchCreateEntities(IntsArchetype, IntEntitiesToSpawnCount, Entities);
		CHECK(ObtainedContext == CreationContextInt);
		++NumDifferentArchetypesUsed;
	}

	CHECK(ObtainedContext->DebugAreEntityCollectionsUpToDate());

	{
		TSharedRef<FMassEntityManager::FEntityCreationContext> TempContext = EntityManager->BatchCreateEntities(IntsArchetype, IntEntitiesToSpawnCount, Entities);
		CHECK(ObtainedContext == TempContext);
		CHECK(TempContext->DebugAreEntityCollectionsUpToDate());
	}

	TArray<FMassArchetypeEntityCollection> EntityCollections = ObtainedContext->GetEntityCollections(*EntityManager.Get());
	CHECK(EntityCollections.Num() == NumDifferentArchetypesUsed);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::CreationContext::ManualBuild", "[Mass][CreationContext]")
{
	constexpr int32 FloatEntitiesToSpawnCount = 7;
	int32 NumDifferentArchetypesUsed = 0;

	TArray<FTestFragment_Float> Payload;
	for (int32 Index = 0; Index < FloatEntitiesToSpawnCount; ++Index)
	{
		Payload.Add(FTestFragment_Float(static_cast<float>(Index)));
	}

	TSharedRef<FMassEntityManager::FEntityCreationContext> ObtainedContext = EntityManager->GetOrMakeCreationContext();

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchReserveEntities(FloatEntitiesToSpawnCount, Entities);

	FStructArrayView PayloadView(Payload);
	TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
	FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(*EntityManager, Entities, FMassArchetypeEntityCollection::NoDuplicates
		, FMassGenericPayloadView(MakeArrayView(&PayloadView, 1)), EntityCollections);

	INFO("We expect TargetEntities to only contain archetype-less entities, ones that need to be built");
	REQUIRE(EntityCollections.Num() == 1);

	{
		TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchBuildEntities(EntityCollections[0], FMassFragmentBitSet(FTestFragment_Float::StaticStruct()));
		CHECK(ObtainedContext == CreationContext);
		++NumDifferentArchetypesUsed;
	}

	CHECK(ObtainedContext->DebugAreEntityCollectionsUpToDate());

	TArray<FMassArchetypeEntityCollection> ContextEntityCollections = ObtainedContext->GetEntityCollections(*EntityManager.Get());
	CHECK(ContextEntityCollections.Num() == NumDifferentArchetypesUsed);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
