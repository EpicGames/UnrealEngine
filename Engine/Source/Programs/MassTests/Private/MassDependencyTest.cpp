// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassProcessorDependencySolver.h"
#include "MassEntityLinkFragments.h"
#include "MassTypeManager.h"

#include "TestHarness.h"
#include "TestMacros/Assertions.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

template<typename T>
static FName GetProcessorName()
{
	return T::StaticClass()->GetFName();
}

//-----------------------------------------------------------------------------
// FDependencySolverFixture — extends FMassLLTFixture with dependency solver helpers
//-----------------------------------------------------------------------------
struct FDependencySolverFixture : FMassLLTFixture
{
	TArray<UMassLLTProcessorBase*> Processors;
	TArray<FMassProcessorOrderInfo> Result;

	FDependencySolverFixture()
	{
		EntityManager->GetTypeManager().RegisterType<FTestSharedFragment_Int>();
		EntityManager->GetTypeManager().RegisterType<UMassLLTWorldSubsystem>();
		EntityManager->GetTypeManager().RegisterType<UMassLLTParallelSubsystem>();
	}

	void Solve()
	{
		Result.Reset();
		FMassProcessorDependencySolver Solver(MakeArrayView((UMassProcessor**)Processors.GetData(), Processors.Num()));
		Solver.ResolveDependencies(Result, EntityManager);
	}
};

//-----------------------------------------------------------------------------
// Tests
//-----------------------------------------------------------------------------

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::Trivial", "[Mass][Dependency]")
{
	UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));

	Solve();

	INFO("The results should contain only a single processor");
	REQUIRE(Result.Num() == 1);
	INFO("The sole processor should be the one we've added");
	CHECK(Result[0].Processor == Proc);
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::Simple", "[Mass][Dependency]")
{
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassLLTProcessor_C>());
	}

	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassLLTProcessor_A>());
	}

	Processors.Add(NewTestProcessor<UMassLLTProcessor_C>(EntityManager));

	Solve();

	INFO("C is expected to be first");
	CHECK(Result[0].Name == GetProcessorName<UMassLLTProcessor_C>());
	INFO("A is expected to be second");
	CHECK(Result[1].Name == GetProcessorName<UMassLLTProcessor_A>());
	INFO("B is expected to be third");
	CHECK(Result[2].Name == GetProcessorName<UMassLLTProcessor_B>());
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::MissingDependencies", "[Mass][Dependency]")
{
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(TEXT("NonExistingDependency"));
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteBefore.Add(TEXT("NonExistingDependency2"));
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassLLTProcessor_C>());
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_C>(EntityManager));
		(void)Proc;
	}

	Solve();

	// even though there's no direct dependency between A and B due to declared dependencies on "NonExistingDependency"
	// B should come before A

	INFO("C is expected to be the first one");
	CHECK(Result[0].Name == GetProcessorName<UMassLLTProcessor_C>());
	INFO("Then B");
	CHECK(Result[1].Name == GetProcessorName<UMassLLTProcessor_B>());
	INFO("With A being last");
	CHECK(Result[2].Name == GetProcessorName<UMassLLTProcessor_A>());
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::DeepGroup", "[Mass][Dependency]")
{
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(TEXT("W.X.Y.Z"));
		Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("P.Q.R");
	}

	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("W.X.Y.Z");
	}

	Solve();

	// dump all the group information from the Result collection for easier ordering testing
	for (int32 ResultIndex = 0; ResultIndex < Result.Num(); ++ResultIndex)
	{
		if (Result[ResultIndex].NodeType != FMassProcessorOrderInfo::EDependencyNodeType::Processor)
		{
			Result.RemoveAt(ResultIndex--, EAllowShrinking::No);
		}
	}

	INFO("B is expected to be first");
	CHECK(Result[0].Name == GetProcessorName<UMassLLTProcessor_B>());
	INFO("A is expected to be second");
	CHECK(Result[1].Name == GetProcessorName<UMassLLTProcessor_A>());
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::Complex", "[Mass][Dependency]")
{
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("X.Z");
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(TEXT("X.Y"));
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(UMassLLTProcessor_E::StaticClass()->GetFName());
	}

	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("X.Y");
	}

	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_C>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("X.Y");
	}

	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_D>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteBefore.Add(UMassLLTProcessor_A::StaticClass()->GetFName());
		Proc->GetMutableExecutionOrder().ExecuteBefore.Add(TEXT("X.Y"));
	}

	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_E>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("X.Z");
	}

	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_F>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(UMassLLTProcessor_A::StaticClass()->GetFName());
	}

	Solve();

	INFO("None of the processors should have been pruned");
	REQUIRE(Result.Num() == Processors.Num());

	for (int32 ResultIndex = 0; ResultIndex < Result.Num(); ++ResultIndex)
	{
		INFO("We expect only processor nodes in the results");
		CHECK(Result[ResultIndex].NodeType == FMassProcessorOrderInfo::EDependencyNodeType::Processor);
	}

	INFO("D is the only fully dependency-less processor so should be first");
	CHECK(Result[0].Name == GetProcessorName<UMassLLTProcessor_D>());
	INFO("B and C come next");
	{
		const bool bBInExpectedSlot = (Result[1].Name == GetProcessorName<UMassLLTProcessor_B>() || Result[2].Name == GetProcessorName<UMassLLTProcessor_B>());
		const bool bCInExpectedSlot = (Result[1].Name == GetProcessorName<UMassLLTProcessor_C>() || Result[2].Name == GetProcessorName<UMassLLTProcessor_C>());
		CHECK(bBInExpectedSlot);
		CHECK(bCInExpectedSlot);
	}
	INFO("Following by E");
	CHECK(Result[3].Name == GetProcessorName<UMassLLTProcessor_E>());
	INFO("Then A");
	CHECK(Result[4].Name == GetProcessorName<UMassLLTProcessor_A>());
	INFO("F is last");
	CHECK(Result[5].Name == GetProcessorName<UMassLLTProcessor_F>());
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::ThreadUnsafeWriteLinkedEntityFragment", "[Mass][Dependency]")
{
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		// This creates the indirect fragment requirement but an indirect requirement alone is not valid
		// The query requirements are valid because it also adds a requirement for the link type (const shared fragment)
		Proc->EntityQuery.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadWrite);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_E>(EntityManager));
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->EntityQuery.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadWrite);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_C>(EntityManager));
		Proc->EntityQuery.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadWrite);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_D>(EntityManager));
		Proc->EntityQuery.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadWrite);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_F>(EntityManager));
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
	}

	Solve();

	int32 EmptyCount = 0;
	for (const FMassProcessorOrderInfo& OrderInfo : Result)
	{
		if (OrderInfo.Dependencies.IsEmpty())
		{
			EmptyCount++;
		}
	}

	INFO("Dependent processor chain should have exactly one independent processor");
	CHECK(EmptyCount == 1);
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::ThreadUnsafeWriteIndirectFragment", "[Mass][Dependency]")
{
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_E>(EntityManager));
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		// This creates ONLY the indirect fragment requirement but an indirect requirement alone is not valid
		Proc->EntityQuery.AddIndirectFragmentRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		// we must have another requirement of some sort, in practice we would want to access something with the entity handle for indirect access
		// making this read-only ensures that the solver is only resolving indirect contention on FTestFragment_Int
		Proc->EntityQuery.AddRequirement<FTestFragment_EntityHandle>(EMassFragmentAccess::ReadOnly);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->EntityQuery.AddIndirectFragmentRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		Proc->EntityQuery.AddRequirement<FTestFragment_EntityHandle>(EMassFragmentAccess::ReadOnly);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_C>(EntityManager));
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
	}

	Solve();

	int32 EmptyCount = 0;
	for (const FMassProcessorOrderInfo& OrderInfo : Result)
	{
		if (OrderInfo.Dependencies.IsEmpty())
		{
			EmptyCount++;
		}
	}

	INFO("Dependent processor chain should have exactly one independent processor");
	CHECK(EmptyCount == 1);
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::ThreadUnsafeWriteSubsystem", "[Mass][Dependency]")
{
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		Proc->EntityQuery.AddSubsystemRequirement<UMassLLTWorldSubsystem>(EMassFragmentAccess::ReadWrite);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->EntityQuery.AddSubsystemRequirement<UMassLLTWorldSubsystem>(EMassFragmentAccess::ReadWrite);
	}

	Solve();

	INFO("Dependency between processors is expected");
	CHECK(Result[0].Dependencies.IsEmpty() != Result[1].Dependencies.IsEmpty());
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::ThreadSafeWriteSubsystem", "[Mass][Dependency]")
{
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		Proc->EntityQuery.AddSubsystemRequirement<UMassLLTParallelSubsystem>(EMassFragmentAccess::ReadWrite);
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->EntityQuery.AddSubsystemRequirement<UMassLLTParallelSubsystem>(EMassFragmentAccess::ReadWrite);
	}

	Solve();

	INFO("No dependency between processors is expected");
	CHECK((Result[0].Dependencies.IsEmpty() && Result[1].Dependencies.IsEmpty()));
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::BadInput::Empty", "[Mass][Dependency]")
{
	Solve();
	INFO("Empty input should be handled gracefully");
	CHECK(Result.Num() == 0);
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::BadInput::SingleNull", "[Mass][Dependency]")
{
	Processors.Reset();
	Processors.Add(nullptr);
	// Production code logs UE_LOG(Warning) and skips nullptr entries — no check/ensure fires
	Solve();
	INFO("Single nullptr input should be handled gracefully");
	CHECK(Result.Num() == 0);
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::BadInput::MultipleNulls", "[Mass][Dependency]")
{
	Processors.Reset();
	Processors.Add(nullptr);
	Processors.Add(nullptr);
	Processors.Add(nullptr);
	// Production code logs UE_LOG(Warning) and skips nullptr entries — no check/ensure fires
	Solve();
	INFO("Multiple nullptr inputs should be handled gracefully");
	CHECK(Result.Num() == 0);
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::BadInput::NullsMixedIn", "[Mass][Dependency]")
{
	Processors.Reset();
	Processors.Add(nullptr);
	Processors.Add(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
	Processors.Add(nullptr);
	Processors.Add(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
	// Production code logs UE_LOG(Warning) and skips nullptr entries — no check/ensure fires
	Solve();
	INFO("Mixed nullptr and proper inputs should be handled gracefully");
	CHECK(Result.Num() == 2);
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::BadInput::Duplicates", "[Mass][Dependency]")
{
	Processors.Reset();
	Processors.Add(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
	Processors.Add(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
	Processors.Add(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
	// Production code logs UE_LOG(Warning) for duplicates — no check/ensure fires
	Solve();
	INFO("Duplicates in input should be handled gracefully");
	CHECK(Result.Num() == 1);
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::SubgroupNames", "[Mass][Dependency]")
{
	TArray<FString> SubGroupNames;
	FName EmptyName;
	FMassProcessorDependencySolver::CreateSubGroupNames(EmptyName, SubGroupNames);

	INFO("Empty group name is supported");
	REQUIRE(SubGroupNames.Num() > 0);
	INFO("Empty group name handled like any other name");
	CHECK(SubGroupNames[0] == EmptyName.ToString());

	FMassProcessorDependencySolver::CreateSubGroupNames(TEXT("X"), SubGroupNames);
	INFO("Trivial group name is supported");
	REQUIRE(SubGroupNames.Num() > 0);
	INFO("Trivial group name shouldn't get decorated");
	CHECK(SubGroupNames[0] == TEXT("X"));

	FMassProcessorDependencySolver::CreateSubGroupNames(TEXT("W.X.Y.Z"), SubGroupNames);
	INFO("Complex group name should result in a number of group names equal to group name's depth");
	REQUIRE(SubGroupNames.Num() == 4);
	INFO("Group name W.X.Y.Z should contain subgroup W");
	CHECK(SubGroupNames.Find(TEXT("W")) != INDEX_NONE);
	INFO("Group name W.X.Y.Z should contain subgroup W.X");
	CHECK(SubGroupNames.Find(TEXT("W.X")) != INDEX_NONE);
	INFO("Group name W.X.Y.Z should contain subgroup W.X.Y");
	CHECK(SubGroupNames.Find(TEXT("W.X.Y")) != INDEX_NONE);
	INFO("Group name W.X.Y.Z should contain subgroup W.X.Y.Z");
	CHECK(SubGroupNames.Find(TEXT("W.X.Y.Z")) != INDEX_NONE);
	INFO("Split up of group name W.X.Y.Z should result in a given order");
	CHECK((SubGroupNames[0] == TEXT("W") && SubGroupNames[1] == TEXT("W.X") && SubGroupNames[2] == TEXT("W.X.Y") && SubGroupNames[3] == TEXT("W.X.Y.Z")));
}

TEST_CASE_METHOD(FDependencySolverFixture, "Mass::Dependency::Circular", "[Mass][Dependency]")
{
	TSharedRef<FMassEntityManager> EntityManagerRef = EntityManager.ToSharedRef();

	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		Proc->EntityQuery.Initialize(EntityManagerRef);
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassLLTProcessor_D>());
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->EntityQuery.Initialize(EntityManagerRef);
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassLLTProcessor_A>());
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_C>(EntityManager));
		Proc->EntityQuery.Initialize(EntityManagerRef);
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassLLTProcessor_B>());
	}
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_D>(EntityManager));
		Proc->EntityQuery.Initialize(EntityManagerRef);
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassLLTProcessor_C>());
	}

	// Solve() logs UE_LOG(Error) for circular dependencies — no check/ensure fires
	Solve();

	// every subsequent processor is expected to depend only on the previous one since all the processors use exactly the same resources
	INFO("The first processor has no dependencies");
	CHECK(Result[0].Dependencies.IsEmpty());
	for (int32 ResultIndex = 1; ResultIndex < Result.Num(); ++ResultIndex)
	{
		const FMassProcessorOrderInfo& ResultNode = Result[ResultIndex];
		INFO("The subsequent processors has only one dependency");
		REQUIRE(ResultNode.Dependencies.Num() == 1);
		INFO("The subsequent processors depend only on the previous one");
		CHECK(ResultNode.Dependencies[0] == Result[ResultIndex - 1].Name);
	}
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
