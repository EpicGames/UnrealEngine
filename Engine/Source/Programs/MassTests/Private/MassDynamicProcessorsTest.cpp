// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

//----------------------------------------------------------------------//
// DynamicProcessors.Add — add a processor dynamically after 1 tick,
// verify it gets ticked the expected number of times.
//----------------------------------------------------------------------//
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::ProcessingPhases::DynamicProcessors::Add", "[Mass][ProcessingPhases][DynamicProcessors]")
{
	const int32 AllowedTicksCount = 2;
	const int32 AddDynamicProcessorOnTickIndex = 1;
	int32 NumberOfTimesTicked = 0;
	UMassLLTProcessorBase* DynamicProcessor = nullptr;
	TWeakObjectPtr<UMassLLTProcessorBase> WeakDynamicProcessor;

	// No processors initially
	InitializePhaseManager();

	for (int32 TickIdx = 0; TickIdx <= AllowedTicksCount + AddDynamicProcessorOnTickIndex; ++TickIdx)
	{
		Tick();

		if (TickIndex == AddDynamicProcessorOnTickIndex)
		{
			DynamicProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
			DynamicProcessor->ExecutionFunction = [&NumberOfTimesTicked](FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
			{
				++NumberOfTimesTicked;
			};

			WaitForCompletion();
			PhaseManager->RegisterDynamicProcessor(*DynamicProcessor);
			WeakDynamicProcessor = DynamicProcessor;
		}
	}

	WaitForCompletion();
	INFO("Expecting the dynamic processor to be ticked the predicted number of times");
	CHECK(NumberOfTimesTicked == AllowedTicksCount);
}

//----------------------------------------------------------------------//
// DynamicProcessors.Remove — add a processor, let it tick, then remove it.
//----------------------------------------------------------------------//
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::ProcessingPhases::DynamicProcessors::Remove", "[Mass][ProcessingPhases][DynamicProcessors]")
{
	const int32 AllowedTicksCount = 1;
	const int32 AddDynamicProcessorOnTickIndex = 1;
	const int32 ArbitraryNumberOfTicksAfterRemoval = 2;
	int32 NumberOfTimesTicked = 0;
	UMassLLTProcessorBase* DynamicProcessor = nullptr;

	InitializePhaseManager();

	const int32 TotalTicks = AllowedTicksCount + AddDynamicProcessorOnTickIndex + ArbitraryNumberOfTicksAfterRemoval;
	for (int32 TickIdx = 0; TickIdx <= TotalTicks; ++TickIdx)
	{
		Tick();

		if (TickIndex == AddDynamicProcessorOnTickIndex)
		{
			DynamicProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
			DynamicProcessor->ExecutionFunction = [&NumberOfTimesTicked](FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
			{
				++NumberOfTimesTicked;
			};

			WaitForCompletion();
			PhaseManager->RegisterDynamicProcessor(*DynamicProcessor);
		}

		if (TickIndex == AddDynamicProcessorOnTickIndex + AllowedTicksCount)
		{
			WaitForCompletion();
			PhaseManager->UnregisterDynamicProcessor(*DynamicProcessor);
		}
	}

	WaitForCompletion();
	INFO("Expecting the dynamic processor to be ticked the predicted number of times");
	CHECK(NumberOfTimesTicked == AllowedTicksCount);
}

//----------------------------------------------------------------------//
// DynamicProcessors.AddGCShield — add a processor, force GC every tick,
// verify the processor survives GC (PhaseManager keeps a strong ref).
//----------------------------------------------------------------------//
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::ProcessingPhases::DynamicProcessors::AddGCShield", "[Mass][ProcessingPhases][DynamicProcessors]")
{
	const int32 AllowedTicksCount = 5;
	const int32 AddDynamicProcessorOnTickIndex = 1;
	int32 NumberOfTimesTicked = 0;
	UMassLLTProcessorBase* DynamicProcessor = nullptr;
	TWeakObjectPtr<UMassLLTProcessorBase> WeakDynamicProcessor;

	InitializePhaseManager();

	for (int32 TickIdx = 0; TickIdx <= AllowedTicksCount + AddDynamicProcessorOnTickIndex; ++TickIdx)
	{
		CollectGarbage(RF_NoFlags);

		Tick();

		if (TickIndex == AddDynamicProcessorOnTickIndex)
		{
			DynamicProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
			DynamicProcessor->ExecutionFunction = [&NumberOfTimesTicked](FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
			{
				++NumberOfTimesTicked;
			};

			WaitForCompletion();
			PhaseManager->RegisterDynamicProcessor(*DynamicProcessor);
			WeakDynamicProcessor = DynamicProcessor;
		}
	}

	WaitForCompletion();
	INFO("Expecting the dynamic processor to be ticked the predicted number of times");
	CHECK(NumberOfTimesTicked == AllowedTicksCount);
	INFO("Expecting the dynamic processor to be valid regardless of multiple garbage collections");
	CHECK(WeakDynamicProcessor.Get() != nullptr);
}

//----------------------------------------------------------------------//
// DynamicProcessors.RemoveGCShield — add, tick, remove, then force GC.
// After removal, the processor should be collected by GC.
//----------------------------------------------------------------------//
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::ProcessingPhases::DynamicProcessors::RemoveGCShield", "[Mass][ProcessingPhases][DynamicProcessors]")
{
	const int32 AllowedTicksCount = 5;
	const int32 AddDynamicProcessorOnTickIndex = 1;
	const int32 ArbitraryNumberOfTicksAfterRemoval = 2;
	int32 NumberOfTimesTicked = 0;
	UMassLLTProcessorBase* DynamicProcessor = nullptr;
	TWeakObjectPtr<UMassLLTProcessorBase> WeakDynamicProcessor;

	InitializePhaseManager();

	const int32 TotalTicks = AllowedTicksCount + AddDynamicProcessorOnTickIndex + ArbitraryNumberOfTicksAfterRemoval;
	for (int32 TickIdx = 0; TickIdx <= TotalTicks; ++TickIdx)
	{
		CollectGarbage(RF_NoFlags);

		Tick();

		if (TickIndex == AddDynamicProcessorOnTickIndex)
		{
			DynamicProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
			DynamicProcessor->ExecutionFunction = [&NumberOfTimesTicked](FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
			{
				++NumberOfTimesTicked;
			};

			WaitForCompletion();
			PhaseManager->RegisterDynamicProcessor(*DynamicProcessor);
			WeakDynamicProcessor = DynamicProcessor;
		}

		if (TickIndex == AddDynamicProcessorOnTickIndex + AllowedTicksCount)
		{
			WaitForCompletion();
			PhaseManager->UnregisterDynamicProcessor(*DynamicProcessor);
		}
	}

	WaitForCompletion();
	// Final GC pass to ensure the processor is collected now that PhaseManager dropped its strong ref
	CollectGarbage(RF_NoFlags);
	INFO("Expecting the dynamic processor to have been set (test implementation verification)");
	CHECK(DynamicProcessor != nullptr);
	INFO("Expecting the dynamic processor to be removed by GC after we unregister it");
	CHECK(WeakDynamicProcessor.Get() == nullptr);
}

//----------------------------------------------------------------------//
// DynamicProcessors.MultipleInstances — register multiple processor
// instances, verify accumulated value matches prediction.
//----------------------------------------------------------------------//
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::ProcessingPhases::DynamicProcessors::MultipleInstances", "[Mass][ProcessingPhases][DynamicProcessors]")
{
	const int32 AllowedTicksCount = 2;
	const int32 AddDynamicProcessorOnTickIndex = 1;
	const int32 NumberOfProcessorsToInstantiate = 3;
	std::atomic<int32> AccumulatedValue = 0;
	int32 ExpectedAccumulatedValue = 0;

	InitializePhaseManager();

	for (int32 TickIdx = 0; TickIdx <= AllowedTicksCount + AddDynamicProcessorOnTickIndex; ++TickIdx)
	{
		Tick();

		if (TickIndex == 1)
		{
			WaitForCompletion();

			for (int32 ProcessorIndex = 0; ProcessorIndex < NumberOfProcessorsToInstantiate; ++ProcessorIndex)
			{
				UMassLLTProcessorBase* DynamicProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
				DynamicProcessor->SetShouldAllowMultipleInstances(true);
				DynamicProcessor->ExecutionFunction = [&AccumulatedValue, ProcessorIndex](FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
				{
					AccumulatedValue += (ProcessorIndex + 1);
				};
				ExpectedAccumulatedValue += (ProcessorIndex + 1);

				PhaseManager->RegisterDynamicProcessor(*DynamicProcessor);
			}
			ExpectedAccumulatedValue *= AllowedTicksCount;
		}
	}

	WaitForCompletion();
	INFO("Expecting the accumulated value to match the prediction");
	CHECK(AccumulatedValue.load() == ExpectedAccumulatedValue);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
