// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"

#include "MassEntityBuilder.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"

#include "TestHarness.h"
#include "TestMacros/Assertions.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

/**
 * Tests to be added:
 * - test observers triggering as expected, i.e. respecting the construction context
 * - entity grouping
 */

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::SimpleBuild", "[Mass][EntityBuilder]")
{
#if WITH_MASSENTITY_DEBUG
	int32 EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
	int32 SomeCounter = 1;
	int32 EntitiesCreatedThisStep = 0;

	{
		FEntityBuilder EntityBuilder(*EntityManager.Get());
		EntityBuilder.SetForceDeferredCommit(false)
			.Add_GetRef<FTestFragment_Int>(SomeCounter++);
		EntityBuilder.Commit();
		EntitiesCreatedThisStep = 1;
	}
#if WITH_MASSENTITY_DEBUG
	INFO("Number of entities created with basic use");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated == EntitiesCreatedThisStep);
	EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
	{
		FEntityBuilder EntityBuilder(*EntityManager.Get());
		EntityBuilder.SetForceDeferredCommit(false);

		ON_SCOPE_EXIT
		{
			EntityBuilder.Commit();
		};

		EntityBuilder.Add_GetRef<FTestFragment_Int>(SomeCounter++);
	}
	EntitiesCreatedThisStep = 1;
#if WITH_MASSENTITY_DEBUG
	INFO("Number of entities created with ON_SCOPE_EXIT");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated == EntitiesCreatedThisStep);
	EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
	{
		FScopedEntityBuilder EntityBuilder(*EntityManager.Get());
		EntityBuilder.SetForceDeferredCommit(false)
			.Add_GetRef<FTestFragment_Int>(SomeCounter++);
	}
	EntitiesCreatedThisStep = 1;
#if WITH_MASSENTITY_DEBUG
	INFO("Number of entities created with scoped builder");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated == EntitiesCreatedThisStep);
	EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
	{
		FScopedEntityBuilder EntityBuilder(*EntityManager.Get());
		EntityBuilder.SetForceDeferredCommit(false)
			.Add_GetRef<FTestFragment_Int>(SomeCounter++);

		FEntityBuilder EntityBuilder2 = EntityBuilder;
		EntityBuilder2.Commit();
	}
	EntitiesCreatedThisStep = 2;
#if WITH_MASSENTITY_DEBUG
	INFO("Number of entities created with with a scoped builder and its regular copy");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated == EntitiesCreatedThisStep);
	EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
	{
		FEntityBuilder EntityBuilder(*EntityManager.Get());
		EntityBuilder.SetForceDeferredCommit(false)
			.Add_GetRef<FTestFragment_Int>(SomeCounter++);
		EntityBuilder.Commit();

		FEntityBuilder EntityBuilder2 = EntityBuilder;
	}
	EntitiesCreatedThisStep = 1;
#if WITH_MASSENTITY_DEBUG
	INFO("Number of entities created with Commit and an abandoned copy of a builder");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated == EntitiesCreatedThisStep);
	EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
	{
		FEntityBuilder EntityBuilder(*EntityManager.Get());
		EntityBuilder.SetForceDeferredCommit(false)
			.Add_GetRef<FTestFragment_Int>(SomeCounter++);

		FEntityBuilder EntityBuilder2 = EntityBuilder;
		EntityBuilder2.Commit();

		EntityBuilder.Reset();
	}
	EntitiesCreatedThisStep = 1;
#if WITH_MASSENTITY_DEBUG
	INFO("Number of entities created with committed copy and abandoned original builder");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated == EntitiesCreatedThisStep);
	EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::Abort", "[Mass][EntityBuilder]")
{
	{
		FEntityBuilder Builder(EntityManager.ToSharedRef());
		Builder.Add<FTestFragment_Int>();
		FMassEntityHandle ReservedEntityHandle = Builder.GetEntityHandle();
		{
			const bool bValid = EntityManager->IsEntityValid(ReservedEntityHandle);
			INFO("Before committing the entity handle is reserved");
			CHECK(bValid);
			const bool bIsBuilt = EntityManager->IsEntityActive(ReservedEntityHandle);
			INFO("Before committing the entity is already created");
			CHECK_FALSE(bIsBuilt);
		}
		Builder.Reset();
		{
			const bool bValid = EntityManager->IsEntityValid(ReservedEntityHandle);
			INFO("After resetting the entity handle is still valid");
			CHECK_FALSE(bValid);
		}
	}
	FMassEntityHandle AbandonedEntityHandle;
	{
		FEntityBuilder Builder(EntityManager.ToSharedRef());
		Builder.Add<FTestFragment_Int>();
		AbandonedEntityHandle = Builder.GetEntityHandle();
	}
	{
		const bool bValid = EntityManager->IsEntityValid(AbandonedEntityHandle);
		INFO("After builder's destruction without committing the entity handle is still valid");
		CHECK_FALSE(bValid);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::OneLiner", "[Mass][EntityBuilder]")
{
	int32 TotalCountCreated = 0;
	{
		FMassEntityHandle CreatedEntity = EntityManager->MakeEntityBuilder()
			.SetForceDeferredCommit(false)
			.Add<FTestFragment_Int>().Commit();
		++TotalCountCreated;

		const bool bIsBuilt = EntityManager->IsEntityActive(CreatedEntity);
		INFO("The entity has been created");
		CHECK(bIsBuilt);
#if WITH_MASSENTITY_DEBUG
		INFO("Only a single entity has been created");
		CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == TotalCountCreated);
#endif // WITH_MASSENTITY_DEBUG
	}
	{
		FEntityBuilder EntityBuilder = EntityManager->MakeEntityBuilder()
			.SetForceDeferredCommit(false)
			.Add<FTestFragment_Int>();
		EntityBuilder.Commit();
		++TotalCountCreated;
	}
#if WITH_MASSENTITY_DEBUG
	INFO("The number of entities created matches expectations");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == TotalCountCreated);
#endif // WITH_MASSENTITY_DEBUG
	{
		// we're not committing so this builder won't create an entity.
		EntityManager->MakeEntityBuilder().Add<FTestFragment_Int>();

		// similarly here, even reserving the entity won't result in building that entity without manual `Commit` call.
		FMassEntityHandle ReservedEntity = EntityManager->MakeEntityBuilder().Add<FTestFragment_Int>().GetEntityHandle();
	}
#if WITH_MASSENTITY_DEBUG
	INFO("The number of entities created after not committing builders");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == TotalCountCreated);
#endif // WITH_MASSENTITY_DEBUG
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::Copy", "[Mass][EntityBuilder]")
{
	const int32 ValueA = 1;
	FEntityBuilder BuilderA(*EntityManager.Get());
	BuilderA.SetForceDeferredCommit(false);
	BuilderA.Add<FTestFragment_Int>(ValueA);

	const int32 ValueB = ValueA + 1;
	FEntityBuilder BuilderB = EntityManager->MakeEntityBuilder();
	BuilderB.SetForceDeferredCommit(false);
	BuilderB.Add<FTestFragment_Int>(ValueB);

	// a different way of setting the value
	FEntityBuilder BuilderC = BuilderA;
	BuilderC.GetOrCreate<FTestFragment_Int>().Value = ValueB;

	BuilderA.Commit();
	BuilderB.Commit();
	BuilderC.Commit();

	FTestFragment_Int* FragmentA = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(BuilderA.GetEntityHandle());
	INFO("The original entity has the expected fragment");
	REQUIRE(FragmentA != nullptr);
	INFO("The value of the original entity's fragment matches expectations");
	CHECK(FragmentA->Value == ValueA);

	FTestFragment_Int* FragmentB = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(BuilderB.GetEntityHandle());
	INFO("The copied entity has the expected fragment");
	REQUIRE(FragmentB != nullptr);
	INFO("The value of the copied entity's fragment matches expectations");
	CHECK(FragmentB->Value == ValueB);

	FTestFragment_Int* FragmentC = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(BuilderC.GetEntityHandle());
	INFO("The other copied entity has the expected fragment");
	REQUIRE(FragmentC != nullptr);
	INFO("The value of the other copied entity's fragment matches expectations");
	CHECK(FragmentC->Value == FragmentB->Value);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::Override", "[Mass][EntityBuilder]")
{
	FEntityBuilder BuilderA(*EntityManager.Get());
	BuilderA.Add<FTestFragment_Int>();

	FEntityBuilder BuilderB(*EntityManager.Get());
	BuilderB.Add<FTestFragment_Float>();

	FEntityBuilder BuilderC = BuilderB;

	FMassEntityHandle EntityA = BuilderA.GetEntityHandle();
	FMassEntityHandle EntityB = BuilderB.GetEntityHandle();
	FMassEntityHandle EntityC = BuilderC.GetEntityHandle();

	INFO("Entities reserved by different builders, A|B");
	CHECK_FALSE(EntityA == EntityB);
	INFO("Entities reserved by different builders, A|C");
	CHECK_FALSE(EntityA == EntityC);
	INFO("Entities reserved by different builders, B|C");
	CHECK_FALSE(EntityB == EntityC);

	// the following operation is expected to stomp the settings of the target builder, but not the entity
	BuilderB = BuilderA;
	FMassEntityHandle EntityB2 = BuilderB.GetEntityHandle();
	INFO("The uncommitted (i.e. reserved) entity handle does not change with builder's config override");
	CHECK(EntityB == EntityB2);

	// overriding a committed builder results in creation of a new handle.
	BuilderC.Commit();
	BuilderC = BuilderA;
	FMassEntityHandle EntityC2 = BuilderC.GetEntityHandle();
	INFO("The committed entity handle differs from the new one");
	CHECK(EntityC != EntityC2);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::PassOver", "[Mass][EntityBuilder]")
{
	{
		FEntityBuilder BuilderA(*EntityManager.Get());
		BuilderA.Add<FTestFragment_Int>();
		// forces handle reservation
		FMassEntityHandle EntityA = BuilderA.GetEntityHandle();

		FMassEntityHandle EntityB;
		{
			FEntityBuilder BuilderB(*EntityManager.Get());
			BuilderB.Add<FTestFragment_Float>();
			EntityB = BuilderB.GetEntityHandle();
			BuilderA = MoveTemp(BuilderB);
		}

		// at this point EntityB should be valid while the original EntityA not
		INFO("The original entity is invalid");
		CHECK(EntityManager->IsEntityValid(EntityA) == false);
		INFO("The passed-over entity entity is valid");
		CHECK(EntityManager->IsEntityValid(EntityB));
	}

	{
		FEntityBuilder BuilderA(*EntityManager.Get());
		BuilderA.SetForceDeferredCommit(false);
		BuilderA.Add<FTestFragment_Int>();
		// forces handle reservation
		FMassEntityHandle EntityA = BuilderA.Commit();

		FMassEntityHandle EntityB;
		{
			FEntityBuilder BuilderB(*EntityManager.Get());
			BuilderB.Add<FTestFragment_Float>();
			EntityB = BuilderB.GetEntityHandle();
			BuilderA = MoveTemp(BuilderB);
		}

		// at this point both EntityA and EntityB should be valid
		INFO("The original entity is valid, since it was committed");
		CHECK(EntityManager->IsEntityValid(EntityA));
		INFO("The original entity is active");
		CHECK(EntityManager->IsEntityActive(EntityA));
		INFO("The passed-over entity entity is valid");
		CHECK(EntityManager->IsEntityValid(EntityB));
	}

	{
		FEntityBuilder BuilderA(*EntityManager.Get());
		BuilderA.Add<FTestFragment_Int>();
		// forces handle reservation
		FMassEntityHandle EntityA = BuilderA.GetEntityHandle();

		FMassEntityHandle EntityB;
		{
			FEntityBuilder BuilderB(*EntityManager.Get());
			BuilderB.SetForceDeferredCommit(false);
			BuilderB.Add<FTestFragment_Float>();
			EntityB = BuilderB.Commit();
			BuilderA = MoveTemp(BuilderB);
		}

		// at this point both EntityA and EntityB should be valid, but only EntityB should be active (commited).
		INFO("The original entity is valid");
		CHECK(EntityManager->IsEntityValid(EntityA));
		INFO("The original entity is NOT active");
		CHECK(EntityManager->IsEntityActive(EntityA) == false);
		INFO("The secondary entity entity is valid");
		CHECK(EntityManager->IsEntityValid(EntityB));
		INFO("The secondary entity is active");
		CHECK(EntityManager->IsEntityActive(EntityB));
	}

	{
		FEntityBuilder BuilderA(*EntityManager.Get());
		BuilderA.SetForceDeferredCommit(false)
			.Add<FTestFragment_Int>();
		// forces handle reservation
		FMassEntityHandle EntityA = BuilderA.Commit();

		FMassEntityHandle EntityB;
		{
			FEntityBuilder BuilderB(*EntityManager.Get());
			BuilderB.SetForceDeferredCommit(false)
				.Add<FTestFragment_Float>();
			EntityB = BuilderB.Commit();
			BuilderA = MoveTemp(BuilderB);
		}

		// at this point both EntityA and EntityB should be valid and active (commited).
		INFO("The original entity is valid");
		CHECK(EntityManager->IsEntityValid(EntityA));
		INFO("The original entity is active");
		CHECK(EntityManager->IsEntityActive(EntityA));
		INFO("The secondary entity entity is valid");
		CHECK(EntityManager->IsEntityValid(EntityB));
		INFO("The secondary entity is active");
		CHECK(EntityManager->IsEntityActive(EntityB));

		// ensureMsgf at MassEntityBuilder.cpp:278 fires only once per process (one-time guard).
		// SyncBuildingAsyncSubmission test typically consumes it first. Use FEnsureScope to catch if it does fire.
		{
			FEnsureScope EnsureScope;
			BuilderA.Commit();
		}

		FMassEntityHandle EntityA2 = BuilderA.GetEntityHandle();
		INFO("The entity handle is the same as the builder's that has been moved");
		CHECK(EntityA2 == EntityB);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::Reuse", "[Mass][EntityBuilder]")
{
	FEntityBuilder Builder = EntityManager->MakeEntityBuilder();
	Builder.SetForceDeferredCommit(false)
		.Add<FTestFragment_Int>();

	auto TestBuilder = [this, &Builder](const FMassArchetypeHandle& ExpectedArchetype)
	{
		TArray<const FMassEntityHandle> Entities = {
			Builder.CommitAndReprepare()
			, Builder.CommitAndReprepare()
		};

		const FMassArchetypeHandle LocalArchetype = Builder.GetArchetypeHandle();
		INFO("Two entities created sequentially");
		CHECK_FALSE(Entities[0] == Entities[1]);
		INFO("Entities' archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entities[0]) == EntityManager->GetArchetypeForEntity(Entities[1]));
		INFO("Builders archetype and entities' archetype");
		CHECK(Builder.GetArchetypeHandle() == EntityManager->GetArchetypeForEntity(Entities[0]));
		INFO("Archetype matches expectations");
		CHECK(EntityManager->GetArchetypeForEntity(Entities[0]) == ExpectedArchetype);
	};

	TestBuilder(IntsArchetype);
	Builder.Add<FTestFragment_Float>();
	TestBuilder(FloatsIntsArchetype);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::DuringProcessing", "[Mass][EntityBuilder]")
{
	constexpr int32 NumIterations = 5;
	// creating a single entity to enforce the execution function of the processor we're going to use to execute
	// exactly once
	EntityManager->CreateEntity(IntsArchetype);

	TArray<FMassEntityHandle> EntityHandles;

	constexpr int32 InitialValueToSet = 100;
	int32 ValueToSet = InitialValueToSet;
	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	Processor->ForEachEntityChunkExecutionFunction = [&ValueToSet, &EntityHandles](FMassExecutionContext& Context)
	{
		FEntityBuilder AsyncBuilder = Context.GetEntityManagerChecked().MakeEntityBuilder();
		AsyncBuilder.Add<FTestFragment_Int>(ValueToSet++);
		EntityHandles.Add(AsyncBuilder.Commit());
	};
	Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);

	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f, /*bFlushCommandBuffer=*/false);

	for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
	{
		Executor::RunProcessorsView(MakeArrayView(reinterpret_cast<UMassProcessor**>(&Processor), 1), ProcessingContext);
		INFO("Number of entities after iteration " << Iteration);
		REQUIRE(EntityHandles.Num() == Iteration + 1);
	}

	for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
	{
		INFO("(NOT) Entity " << Iteration << " is `created`");
		CHECK_FALSE(EntityManager->IsEntityBuilt(EntityHandles[Iteration]));
	}

	EntityManager->FlushCommands();

	for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
	{
		INFO("Entity " << Iteration << " is `created`");
		CHECK(EntityManager->IsEntityBuilt(EntityHandles[Iteration]));
		INFO("Entity " << Iteration << " has the right archetype");
		CHECK(EntityManager->GetArchetypeForEntity(EntityHandles[Iteration]) == IntsArchetype);
		if (Iteration + 1 < NumIterations)
		{
			INFO("(NOT) Entity handles are the same " << Iteration);
			CHECK_FALSE(EntityHandles[Iteration] == EntityHandles[Iteration + 1]);
		}
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::SyncBuildingAsyncSubmission", "[Mass][EntityBuilder]")
{
	// creating a single entity to enforce the execution function of the processor we're going to use to execute
	// exactly once
	EntityManager->CreateEntity(IntsArchetype);

	FEntityBuilder SyncBuilder = EntityManager->MakeEntityBuilder();
	SyncBuilder.SetForceDeferredCommit(false)
		.Add<FTestFragment_Int>();

	const FMassEntityHandle ReservedHandle = SyncBuilder.GetEntityHandle();

	int32 ProcessedEntitiesCount = 0;
	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	Processor->ForEachEntityChunkExecutionFunction = [&SyncBuilder, &ProcessedEntitiesCount](FMassExecutionContext& Context)
	{
		SyncBuilder.Commit();
		ProcessedEntitiesCount += Context.GetNumEntities();
	};
	Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);

	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f, /*bFlushCommandBuffer=*/false);
	Executor::RunProcessorsView(MakeArrayView(reinterpret_cast<UMassProcessor**>(&Processor), 1), ProcessingContext);

	INFO("Number of fully-formed entities expected");
	CHECK(ProcessedEntitiesCount == 1);
	INFO("The entity handle before and after async commit");
	CHECK(ReservedHandle == SyncBuilder.GetEntityHandle());
	INFO("The Builder is in `Committed` state");
	CHECK(SyncBuilder.IsCommitted());
	// since the commands are not flushed yet, due to ProcessingContext's values, we expect the entity to not be created yet
	INFO("(NOT) the entity has been created");
	CHECK_FALSE(EntityManager->IsEntityBuilt(ReservedHandle));

	// Second execution of the processor shouldn't change a thing — the commit triggers an ensure
	// because it's trying to commit an already-committed builder.
	// Note: ensureMsgf at MassEntityBuilder.cpp:278 fires only once per process. PassOver test may
	// have consumed it first. Use FEnsureScope to absorb if it does fire.
	{
		FEnsureScope EnsureScope;
		Executor::RunProcessorsView(MakeArrayView(reinterpret_cast<UMassProcessor**>(&Processor), 1), ProcessingContext);
	}

	INFO("Run 2: The entity handle before and after async commit");
	CHECK(ReservedHandle == SyncBuilder.GetEntityHandle());
	INFO("Run 2: The Builder is in `Committed` state");
	CHECK(SyncBuilder.IsCommitted());
	// since the commands are not flushed yet, due to ProcessingContext's values, we expect the entity to not be created yet
	INFO("(NOT) Run 2: the entity has been created");
	CHECK_FALSE(EntityManager->IsEntityBuilt(ReservedHandle));

	EntityManager->FlushCommands();
#if WITH_MASSENTITY_DEBUG
	INFO("Number of entities in the target archetype, after flushing");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 2);
#endif // WITH_MASSENTITY_DEBUG
	INFO("the entity has been created");
	CHECK(EntityManager->IsEntityBuilt(ReservedHandle));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::AllElements", "[Mass][EntityBuilder]")
{
	const int32 TestIntValue = 17;
	const float TestFloatValue = 3.1415f;
	const float TestSharedFloatValue = 2.71828f;
	const int32 TestSharedIntValue = 1009;

	// quick builder just to create an entity with known properties
	FEntityBuilder Builder(EntityManager.ToSharedRef());
	Builder.SetForceDeferredCommit(false);
	Builder.Add<FTestFragment_Int>(TestIntValue);
	Builder.Add<FTestFragment_Float>(TestFloatValue);
	Builder.Add<FTestTag_B>();
	Builder.Add<FTestSharedFragment_Float>(TestSharedFloatValue);
	Builder.Add<FTestConstSharedFragment_Int>(TestSharedIntValue);

	FMassEntityHandle OriginalEntity = Builder.Commit();

	FMassArchetypeCompositionDescriptor PredictedComposition;
	PredictedComposition.Add<FTestFragment_Int>();
	PredictedComposition.Add<FTestFragment_Float>();
	PredictedComposition.Add<FTestTag_B>();
	PredictedComposition.Add<FTestSharedFragment_Float>();
	PredictedComposition.Add<FTestConstSharedFragment_Int>();

	// testing composition
	const FMassArchetypeHandle ArchetypeHandle = EntityManager->GetArchetypeForEntity(OriginalEntity);
	const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager->GetArchetypeComposition(ArchetypeHandle);
	INFO("Resulting archetype composition matches prediction");
	CHECK(ArchetypeComposition.IsEquivalent(PredictedComposition));

	// test entity fragment values
	{
		FTestFragment_Int* IntFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(OriginalEntity);
		INFO("Created entity has the int fragment");
		REQUIRE(IntFragmentInstance != nullptr);
		INFO("Resulting Int fragment value");
		CHECK(IntFragmentInstance->Value == TestIntValue);
	}
	{
		FTestFragment_Float* FloatFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(OriginalEntity);
		INFO("Created entity has the float fragment");
		REQUIRE(FloatFragmentInstance != nullptr);
		INFO("Resulting Float fragment value");
		CHECK(FloatFragmentInstance->Value == TestFloatValue);
	}
	{
		FTestSharedFragment_Float* SharedFragmentInstance = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(OriginalEntity);
		INFO("Created entity has the shared float fragment");
		REQUIRE(SharedFragmentInstance != nullptr);
		INFO("Resulting shared Float fragment value");
		CHECK(SharedFragmentInstance->Value == TestSharedFloatValue);
	}
	{
		FTestConstSharedFragment_Int* ConstSharedFragmentInstance = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(OriginalEntity);
		INFO("Created entity has the const shared int fragment");
		REQUIRE(ConstSharedFragmentInstance != nullptr);
		INFO("Resulting const shared Int fragment value");
		CHECK(ConstSharedFragmentInstance->Value == TestSharedIntValue);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::CopyEntity", "[Mass][EntityBuilder]")
{
	const int32 TestIntValue = 17;
	const float TestFloatValue = 3.1415f;
	const float TestSharedFloatValue = 2.71828f;
	const int32 TestSharedIntValue = 1009;

	// quick builder just to create an entity with known properties
	FEntityBuilder SetupBuilder(EntityManager.ToSharedRef());
	SetupBuilder.SetForceDeferredCommit(false);
	SetupBuilder.Add<FTestFragment_Int>(TestIntValue);
	SetupBuilder.Add<FTestFragment_Float>(TestFloatValue);
	SetupBuilder.Add<FTestTag_B>();
	SetupBuilder.Add<FTestSharedFragment_Float>(TestSharedFloatValue);
	SetupBuilder.Add<FTestConstSharedFragment_Int>(TestSharedIntValue);

	FMassEntityHandle OriginalEntity = SetupBuilder.Commit();

	// now copy the entity
	FEntityBuilder Builder(EntityManager.ToSharedRef());
	Builder.SetForceDeferredCommit(false);
	Builder.CopyDataFromEntity(OriginalEntity);

	const FMassEntityHandle NewEntityHandle = Builder.Commit();

	INFO("Source and target entities are in the same archetype");
	CHECK(EntityManager->GetArchetypeForEntity(OriginalEntity) == EntityManager->GetArchetypeForEntity(NewEntityHandle));

	// test entity fragment values on the copy
	{
		FTestFragment_Int* IntFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(NewEntityHandle);
		INFO("Copied entity has the int fragment");
		REQUIRE(IntFragmentInstance != nullptr);
		INFO("Resulting Int fragment value");
		CHECK(IntFragmentInstance->Value == TestIntValue);
	}
	{
		FTestFragment_Float* FloatFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(NewEntityHandle);
		INFO("Copied entity has the float fragment");
		REQUIRE(FloatFragmentInstance != nullptr);
		INFO("Resulting Float fragment value");
		CHECK(FloatFragmentInstance->Value == TestFloatValue);
	}
	{
		FTestSharedFragment_Float* SharedFragmentInstance = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(NewEntityHandle);
		INFO("Copied entity has the shared float fragment");
		REQUIRE(SharedFragmentInstance != nullptr);
		INFO("Resulting shared Float fragment value");
		CHECK(SharedFragmentInstance->Value == TestSharedFloatValue);
	}
	{
		FTestConstSharedFragment_Int* ConstSharedFragmentInstance = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(NewEntityHandle);
		INFO("Copied entity has the const shared int fragment");
		REQUIRE(ConstSharedFragmentInstance != nullptr);
		INFO("Resulting const shared Int fragment value");
		CHECK(ConstSharedFragmentInstance->Value == TestSharedIntValue);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::Append", "[Mass][EntityBuilder]")
{
	const int32 TestIntValue = 17;
	const float TestFloatValue = 3.1415f;
	const float TestSharedFloatValue = 2.71828f;
	const int32 TestSharedIntValue = 1009;

	// quick builder just to create an entity with known properties
	FEntityBuilder SetupBuilder(EntityManager.ToSharedRef());
	SetupBuilder.SetForceDeferredCommit(false);
	SetupBuilder.Add<FTestFragment_Int>(TestIntValue);
	SetupBuilder.Add<FTestFragment_Float>(TestFloatValue);
	SetupBuilder.Add<FTestTag_B>();
	SetupBuilder.Add<FTestSharedFragment_Float>(TestSharedFloatValue);
	SetupBuilder.Add<FTestConstSharedFragment_Int>(TestSharedIntValue);

	FMassEntityHandle OriginalEntity = SetupBuilder.Commit();

	// now append from the entity
	FEntityBuilder Builder(EntityManager.ToSharedRef());
	// adding something the appending won't add, just to remove it later and test the result
	Builder.Add<FTestTag_A>();
	Builder.AppendDataFromEntity(OriginalEntity);

	const FMassEntityHandle NewEntityHandle = Builder.SetForceDeferredCommit(false).Commit();

	EntityManager->RemoveTagFromEntity(NewEntityHandle, FTestTag_A::StaticStruct());

	INFO("Source and target entities are in the same archetype");
	CHECK(EntityManager->GetArchetypeForEntity(OriginalEntity) == EntityManager->GetArchetypeForEntity(NewEntityHandle));

	// test entity fragment values on the appended entity
	{
		FTestFragment_Int* IntFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(NewEntityHandle);
		INFO("Appended entity has the int fragment");
		REQUIRE(IntFragmentInstance != nullptr);
		INFO("Resulting Int fragment value");
		CHECK(IntFragmentInstance->Value == TestIntValue);
	}
	{
		FTestFragment_Float* FloatFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(NewEntityHandle);
		INFO("Appended entity has the float fragment");
		REQUIRE(FloatFragmentInstance != nullptr);
		INFO("Resulting Float fragment value");
		CHECK(FloatFragmentInstance->Value == TestFloatValue);
	}
	{
		FTestSharedFragment_Float* SharedFragmentInstance = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(NewEntityHandle);
		INFO("Appended entity has the shared float fragment");
		REQUIRE(SharedFragmentInstance != nullptr);
		INFO("Resulting shared Float fragment value");
		CHECK(SharedFragmentInstance->Value == TestSharedFloatValue);
	}
	{
		FTestConstSharedFragment_Int* ConstSharedFragmentInstance = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(NewEntityHandle);
		INFO("Appended entity has the const shared int fragment");
		REQUIRE(ConstSharedFragmentInstance != nullptr);
		INFO("Resulting const shared Int fragment value");
		CHECK(ConstSharedFragmentInstance->Value == TestSharedIntValue);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityBuilder::WithInstancedStructs", "[Mass][EntityBuilder]")
{
	const int32 TestIntValue = 17;
	const float TestFloatValue = 3.1415f;
	const float TestSharedFloatValue = 2.71828f;
	const int32 TestSharedIntValue = 1009;

	// quick builder just to create an entity with known properties
	FEntityBuilder Builder(EntityManager.ToSharedRef());
	Builder.SetForceDeferredCommit(false);

	FInstancedStruct ElementInstance;

	// adding "wrong value" first to verify that the last "add" will hold
	ElementInstance.InitializeAs<FTestFragment_Int>();
	Builder.Add(ElementInstance);
	ElementInstance.InitializeAs<FTestFragment_Int>(TestIntValue);
	Builder.Add(ElementInstance);

	ElementInstance.InitializeAs<FTestFragment_Float>();
	Builder.Add(MoveTemp(ElementInstance));
	ElementInstance.InitializeAs<FTestFragment_Float>(TestFloatValue);
	Builder.Add(MoveTemp(ElementInstance));

	ElementInstance.InitializeAs<FTestSharedFragment_Float>();
	Builder.Add(MoveTemp(ElementInstance));
	ElementInstance.InitializeAs<FTestSharedFragment_Float>(TestSharedFloatValue);
	Builder.Add(MoveTemp(ElementInstance));

	ElementInstance.InitializeAs<FTestConstSharedFragment_Int>();
	Builder.Add(ElementInstance);
	ElementInstance.InitializeAs<FTestConstSharedFragment_Int>(TestSharedIntValue);
	Builder.Add(ElementInstance);

	// tags cannot be added as instanced structs
	Builder.Add<FTestTag_B>();

	FMassEntityHandle OriginalEntity = Builder.Commit();

	FMassArchetypeCompositionDescriptor PredictedComposition;
	PredictedComposition.Add<FTestFragment_Int>();
	PredictedComposition.Add<FTestFragment_Float>();
	PredictedComposition.Add<FTestTag_B>();
	PredictedComposition.Add<FTestSharedFragment_Float>();
	PredictedComposition.Add<FTestConstSharedFragment_Int>();

	// testing composition
	const FMassArchetypeHandle ArchetypeHandle = EntityManager->GetArchetypeForEntity(OriginalEntity);
	const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager->GetArchetypeComposition(ArchetypeHandle);
	INFO("Resulting archetype composition matches prediction");
	CHECK(ArchetypeComposition.IsEquivalent(PredictedComposition));

	// test entity fragment values
	{
		FTestFragment_Int* IntFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(OriginalEntity);
		INFO("Created entity has the int fragment");
		REQUIRE(IntFragmentInstance != nullptr);
		INFO("Resulting Int fragment value");
		CHECK(IntFragmentInstance->Value == TestIntValue);
	}
	{
		FTestFragment_Float* FloatFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(OriginalEntity);
		INFO("Created entity has the float fragment");
		REQUIRE(FloatFragmentInstance != nullptr);
		INFO("Resulting Float fragment value");
		CHECK(FloatFragmentInstance->Value == TestFloatValue);
	}
	{
		FTestSharedFragment_Float* SharedFragmentInstance = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(OriginalEntity);
		INFO("Created entity has the shared float fragment");
		REQUIRE(SharedFragmentInstance != nullptr);
		INFO("Resulting shared Float fragment value");
		CHECK(SharedFragmentInstance->Value == TestSharedFloatValue);
	}
	{
		FTestConstSharedFragment_Int* ConstSharedFragmentInstance = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(OriginalEntity);
		INFO("Created entity has the const shared int fragment");
		REQUIRE(ConstSharedFragmentInstance != nullptr);
		INFO("Resulting const shared Int fragment value");
		CHECK(ConstSharedFragmentInstance->Value == TestSharedIntValue);
	}
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
