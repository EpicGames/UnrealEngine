// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "Algo/Compare.h"
#include "MassEntityCollection.h"
#include "MassEntityUtils.h"
#include "TestMacros/Assertions.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

//-----------------------------------------------------------------------------
// FEntityCollection tests
//-----------------------------------------------------------------------------
struct FEntityCollectionFixture : FMassLLTEntityFixture
{
	TArray<FMassEntityHandle> FloatEntities;
	TArray<FMassEntityHandle> IntEntities;
	TArray<FMassEntityHandle> FloatIntEntities;
	int32 EntitiesToCreatePerArchetype = 100;
	FRandomStream RandomStream = 0;
	static constexpr int32 NumArchetypesUsed = 3;
	static constexpr int32 NumTestedEntities = 50;

	FEntityCollectionFixture()
	{
		EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesToCreatePerArchetype, FloatEntities);
		EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToCreatePerArchetype, IntEntities);
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, EntitiesToCreatePerArchetype, FloatIntEntities);
	}

	TConstArrayView<FMassEntityHandle> GetEntityArray(const int32 ArrayIndex)
	{
		return ArrayIndex == 2
			? FloatIntEntities
			: (ArrayIndex == 1
				? IntEntities
				: FloatEntities);
	}

	TArray<FMassEntityHandle> CreateEntitySubset(const int32 NumEntities = NumTestedEntities)
	{
		TArray<FMassEntityHandle> EntitiesSubSet;
		int32 ArrayIndex = 0;
		while (EntitiesSubSet.Num() < NumEntities)
		{
			EntitiesSubSet.AddUnique(GetEntityArray(ArrayIndex)[RandomStream.RandRange(0, EntitiesToCreatePerArchetype - 1)]);
			ArrayIndex = (ArrayIndex + 1) % NumArchetypesUsed;
		}
		return EntitiesSubSet;
	}

	TArray<FMassEntityHandle> GetArraySubset(TConstArrayView<FMassEntityHandle> InArray, int32 NumEntities) const
	{
		TArray<FMassEntityHandle> EntitiesSubSet;
		if (NumEntities >= InArray.Num())
		{
			EntitiesSubSet = InArray;
		}
		else
		{
			FRandomStream LocalRand = RandomStream;
			while (EntitiesSubSet.Num() < NumEntities)
			{
				EntitiesSubSet.AddUnique(InArray[LocalRand.RandRange(0, InArray.Num() - 1)]);
			}
		}
		return EntitiesSubSet;
	}

	bool CompareCollectionArrays(TConstArrayView<FMassArchetypeEntityCollection> CollectionsA, TConstArrayView<FMassArchetypeEntityCollection> CollectionsB) const
	{
		for (const FMassArchetypeEntityCollection& ArchetypeCollectionA : CollectionsA)
		{
			const FMassArchetypeEntityCollection* ArchetypeCollectionB = CollectionsB.FindByPredicate([&ArchetypeCollectionA](const FMassArchetypeEntityCollection& Element)
			{
				return Element.IsSameArchetype(ArchetypeCollectionA);
			});
			if (!ArchetypeCollectionB || !ArchetypeCollectionA.IsSame(*ArchetypeCollectionB))
			{
				return false;
			}
		}
		return true;
	}
};

TEST_CASE_METHOD(FEntityCollectionFixture, "Mass::EntityCollection::Empty", "[Mass][EntityCollection]")
{
	FEntityCollection EntityCollection;

	CHECK(EntityCollection.IsEmpty());
	CHECK(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetEntityHandlesView().IsEmpty());
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().IsEmpty());
	CHECK(EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).IsEmpty());
}

TEST_CASE_METHOD(FEntityCollectionFixture, "Mass::EntityCollection::PopulateWithIndividualHandles", "[Mass][EntityCollection]")
{
	FEntityCollection EntityCollection;
	TArray<FMassEntityHandle> EntitiesSubSet = CreateEntitySubset();

	EntityCollection.AddHandle(EntitiesSubSet[0]);
	CHECK_FALSE(EntityCollection.IsEmpty());
	CHECK_FALSE(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetEntityHandlesView().Num() == 1);
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 0);
	CHECK(EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num() == 1);
	CHECK(EntityCollection.IsUpToDate());

	EntityCollection.AddHandle(EntitiesSubSet[1]);
	CHECK_FALSE(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetEntityHandlesView().Num() == 2);
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 0);

	for (int32 HandleIndex = 2; HandleIndex < EntitiesSubSet.Num(); ++HandleIndex)
	{
		EntityCollection.AddHandle(EntitiesSubSet[HandleIndex]);
		CHECK(EntityCollection.GetEntityHandlesView().Num() == HandleIndex + 1);
	}
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 0);
	CHECK(EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num() == NumArchetypesUsed);
	CHECK(EntityCollection.IsUpToDate());

	// verify order-independence
	FEntityCollection SecondEntityCollection;
	for (int32 HandleIndex = EntitiesSubSet.Num() - 1; HandleIndex >= 0; --HandleIndex)
	{
		SecondEntityCollection.AddHandle(EntitiesSubSet[HandleIndex]);
	}

	TConstArrayView<FMassArchetypeEntityCollection> ArchetypeCollections = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get());
	TConstArrayView<FMassArchetypeEntityCollection> SecondArchetypeCollections = SecondEntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get());

	CHECK(CompareCollectionArrays(ArchetypeCollections, SecondArchetypeCollections));
}

TEST_CASE_METHOD(FEntityCollectionFixture, "Mass::EntityCollection::PopulateWithHandleArrays", "[Mass][EntityCollection]")
{
	FEntityCollection EntityCollection;
	TArray<FMassEntityHandle> EntitiesSubSet = CreateEntitySubset();
	REQUIRE(EntitiesSubSet.Num() >= 5);
	TArray<TConstArrayView<FMassEntityHandle>> SubViews = {
		MakeArrayView(&EntitiesSubSet[0], EntitiesSubSet.Num() / 2)
		, MakeArrayView(&EntitiesSubSet[EntitiesSubSet.Num() / 2], 2)
		, MakeArrayView(&EntitiesSubSet[EntitiesSubSet.Num() / 2 + 2], EntitiesSubSet.Num() / 2 - 2)
	};

	EntityCollection.AppendHandles(SubViews[0]);
	CHECK_FALSE(EntityCollection.IsEmpty());
	CHECK_FALSE(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetEntityHandlesView().Num() == SubViews[0].Num());
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 0);
	const int32 NumArchetypesInFirstSlice = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num();
	CHECK((NumArchetypesInFirstSlice > 0 && NumArchetypesInFirstSlice <= NumArchetypesUsed));
	CHECK(EntityCollection.IsUpToDate());

	EntityCollection.AppendHandles(SubViews[1]);
	CHECK_FALSE(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetEntityHandlesView().Num() == SubViews[0].Num() + SubViews[1].Num());
	const int32 NumArchetypesInTwoSlices = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num();
	CHECK(NumArchetypesInTwoSlices >= NumArchetypesInFirstSlice);

	EntityCollection.AppendHandles(SubViews[2]);
	CHECK(EntityCollection.GetEntityHandlesView().Num() == EntitiesSubSet.Num());
	const int32 TotalNumArchetypes = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num();
	CHECK(TotalNumArchetypes == NumArchetypesUsed);
}

TEST_CASE_METHOD(FEntityCollectionFixture, "Mass::EntityCollection::PopulateWithCollections", "[Mass][EntityCollection]")
{
	TArray<FMassEntityHandle> EntitiesSubSet = CreateEntitySubset();
	TArray<FMassArchetypeEntityCollection> ArchetypeCollections;
	Utils::CreateEntityCollections(*EntityManager.Get(), EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates, ArchetypeCollections);
	REQUIRE(ArchetypeCollections.Num() == NumArchetypesUsed);
	static_assert(NumArchetypesUsed >= 3);

	FEntityCollection EntityCollection;
	EntityCollection.AppendCollection(ArchetypeCollections[0]);
	CHECK_FALSE(EntityCollection.IsEmpty());
	CHECK(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 1);

	EntityCollection.AppendCollection(MoveTemp(ArchetypeCollections[1]));
	CHECK(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 2);

	EntityCollection.AppendCollection(ArchetypeCollections[2]);
	CHECK(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 3);

	CHECK(EntityCollection.GetEntityHandlesView().Num() == EntitiesSubSet.Num());

	TArray<FMassEntityHandle> ExportedHandles;
	ExportedHandles.Append(EntityCollection.GetEntityHandlesView());
	ExportedHandles.Sort();
	EntitiesSubSet.Sort();
	CHECK(Algo::Compare(ExportedHandles, EntitiesSubSet));
}

TEST_CASE_METHOD(FEntityCollectionFixture, "Mass::EntityCollection::PopulateWithOutdatedCollections", "[Mass][EntityCollection]")
{
	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> EntitiesSubSet = GetArraySubset(FloatEntities, NumEntities);
	FMassArchetypeEntityCollection InitialCollection = FMassArchetypeEntityCollection(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates);
	CHECK(InitialCollection.IsUpToDate());

	EntityManager->BatchChangeFragmentCompositionForEntities(MakeArrayView(&InitialCollection, 1), FMassFragmentBitSet(FTestFragment_Int::StaticStruct()), {});

	CHECK_FALSE(InitialCollection.IsUpToDate());
	// Verify that the collection is correctly detected as outdated.
	// The full check-failure behavior (testableCheckfReturn) requires WITH_AITESTSUITE
	// to properly short-circuit. Tested in MassEntityTestSuite.
}

TEST_CASE_METHOD(FEntityCollectionFixture, "Mass::EntityCollection::PopulateWithCollectionsHandlesPairs", "[Mass][EntityCollection]")
{
	const int32 SubEntitiesPerArchetype = EntitiesToCreatePerArchetype / 3;
	TArray<TArray<FMassEntityHandle>> PerArchetypeEntitiesSubSet = {
		GetArraySubset(FloatEntities, SubEntitiesPerArchetype)
		, GetArraySubset(IntEntities, SubEntitiesPerArchetype)
		, GetArraySubset(FloatIntEntities, SubEntitiesPerArchetype)
	};

	TArray<FMassArchetypeEntityCollection> ArchetypeCollections = {
		FMassArchetypeEntityCollection(FloatsArchetype, PerArchetypeEntitiesSubSet[0], FMassArchetypeEntityCollection::NoDuplicates)
		, FMassArchetypeEntityCollection(IntsArchetype, PerArchetypeEntitiesSubSet[1], FMassArchetypeEntityCollection::NoDuplicates)
		, FMassArchetypeEntityCollection(FloatsIntsArchetype, PerArchetypeEntitiesSubSet[2], FMassArchetypeEntityCollection::NoDuplicates)
	};

	FEntityCollection EntityCollection;
	EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[0], MoveTemp(ArchetypeCollections[0]));
	CHECK_FALSE(EntityCollection.IsEmpty());
	CHECK(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 1);
	CHECK(EntityCollection.GetEntityHandlesView().Num() == SubEntitiesPerArchetype);

	EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[1], MoveTemp(ArchetypeCollections[1]));
	CHECK(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 2);
	CHECK(EntityCollection.GetEntityHandlesView().Num() == SubEntitiesPerArchetype * 2);

	EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[2], MoveTemp(ArchetypeCollections[2]));
	CHECK(EntityCollection.IsUpToDate());
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 3);
	CHECK(EntityCollection.GetEntityHandlesView().Num() == SubEntitiesPerArchetype * 3);

	TArray<FMassArchetypeEntityCollection> CachedCollections;
	CachedCollections.Append(EntityCollection.GetCachedPerArchetypeCollections());
	CHECK(CompareCollectionArrays(
		CachedCollections
		, EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));
}

TEST_CASE_METHOD(FEntityCollectionFixture, "Mass::EntityCollection::PopulateWithDuplicates", "[Mass][EntityCollection]")
{
	constexpr int32 DuplicatesCount = 5;
	FEntityCollection EntityCollection;
	FMassEntityHandle EntityHandle = FloatEntities[RandomStream.RandRange(0, FloatEntities.Num() - 1)];

	for (int32 Counter = 0; Counter < DuplicatesCount; ++Counter)
	{
		EntityCollection.AddHandle(EntityHandle);
	}

	CHECK(EntityCollection.GetEntityHandlesView().Num() == DuplicatesCount);
	CHECK_FALSE(EntityCollection.IsUpToDate());
	TConstArrayView<FMassArchetypeEntityCollection> Collections = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager);
	REQUIRE(Collections.Num() == 1);
#if WITH_MASSENTITY_DEBUG
	CHECK(Collections[0].DebugCountEntities() == 1);
#endif

	CHECK(EntityCollection.GetEntityHandlesView().Num() == DuplicatesCount);
	CHECK(EntityCollection.UpdateAndRemoveDuplicates(*EntityManager));
	CHECK(EntityCollection.GetEntityHandlesView().Num() == 1);
	CHECK(EntityCollection.GetCachedPerArchetypeCollections().Num() == 1);
}

TEST_CASE_METHOD(FEntityCollectionFixture, "Mass::EntityCollection::MethodEquivalency", "[Mass][EntityCollection]")
{
	const int32 SubEntitiesPerArchetype = EntitiesToCreatePerArchetype / 3;
	TArray<TArray<FMassEntityHandle>> PerArchetypeEntitiesSubSet = {
		GetArraySubset(FloatEntities, SubEntitiesPerArchetype)
		, GetArraySubset(IntEntities, SubEntitiesPerArchetype)
		, GetArraySubset(FloatIntEntities, SubEntitiesPerArchetype)
	};

	TArray<FMassEntityHandle> EntitiesSubSet = PerArchetypeEntitiesSubSet[0];
	EntitiesSubSet.Append(PerArchetypeEntitiesSubSet[1]);
	EntitiesSubSet.Append(PerArchetypeEntitiesSubSet[2]);

	FEntityCollection EntityCollectionFromArray(EntitiesSubSet);
	FEntityCollection EntityCollectionFromHandles;
	for (const FMassEntityHandle& EntityHandle : EntitiesSubSet)
	{
		EntityCollectionFromHandles.AddHandle(EntityHandle);
	}
	CHECK(CompareCollectionArrays(
		EntityCollectionFromArray.GetUpToDatePerArchetypeCollections(*EntityManager.Get())
		, EntityCollectionFromHandles.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));

	FEntityCollection EntityCollectionFromHandlesAndArrays;
	{
		EntityCollectionFromHandlesAndArrays.AddHandle(EntitiesSubSet[0]);
		EntityCollectionFromHandlesAndArrays.AppendHandles(MakeArrayView(&EntitiesSubSet[1], (EntitiesSubSet.Num()) / 2));
		EntityCollectionFromHandlesAndArrays.AddHandle(EntitiesSubSet[EntityCollectionFromHandlesAndArrays.GetEntityHandlesView().Num()]);
		const int32 NumHandlesStoredAlready = EntityCollectionFromHandlesAndArrays.GetEntityHandlesView().Num();
		EntityCollectionFromHandlesAndArrays.AppendHandles(MakeArrayView(&EntitiesSubSet[NumHandlesStoredAlready], EntitiesSubSet.Num() - NumHandlesStoredAlready));
	}
	FEntityCollection EntityCollectionFromArraysAndHandles;
	{
		EntityCollectionFromArraysAndHandles.AppendHandles(MakeArrayView(&EntitiesSubSet[0], (EntitiesSubSet.Num()) / 2));
		EntityCollectionFromArraysAndHandles.AddHandle(EntitiesSubSet[EntityCollectionFromArraysAndHandles.GetEntityHandlesView().Num()]);
		const int32 NumHandlesStoredAlready = EntityCollectionFromArraysAndHandles.GetEntityHandlesView().Num();
		EntityCollectionFromArraysAndHandles.AppendHandles(MakeArrayView(&EntitiesSubSet[NumHandlesStoredAlready], EntitiesSubSet.Num() - NumHandlesStoredAlready));
	}

	CHECK(CompareCollectionArrays(
		EntityCollectionFromHandlesAndArrays.GetUpToDatePerArchetypeCollections(*EntityManager.Get())
		, EntityCollectionFromArraysAndHandles.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));

	CHECK(CompareCollectionArrays(
		EntityCollectionFromHandlesAndArrays.GetUpToDatePerArchetypeCollections(*EntityManager.Get())
		, EntityCollectionFromHandles.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));

	TArray<FMassArchetypeEntityCollection> ArchetypeCollections = {
		FMassArchetypeEntityCollection(FloatsArchetype, PerArchetypeEntitiesSubSet[0], FMassArchetypeEntityCollection::NoDuplicates)
		, FMassArchetypeEntityCollection(IntsArchetype, PerArchetypeEntitiesSubSet[1], FMassArchetypeEntityCollection::NoDuplicates)
		, FMassArchetypeEntityCollection(FloatsIntsArchetype, PerArchetypeEntitiesSubSet[2], FMassArchetypeEntityCollection::NoDuplicates)
	};

	FEntityCollection EntityCollection(PerArchetypeEntitiesSubSet[0], MoveTemp(ArchetypeCollections[0]));
	EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[1], MoveTemp(ArchetypeCollections[1]));
	EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[2], MoveTemp(ArchetypeCollections[2]));

	CHECK(CompareCollectionArrays(
		EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get())
		, EntityCollectionFromHandles.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
