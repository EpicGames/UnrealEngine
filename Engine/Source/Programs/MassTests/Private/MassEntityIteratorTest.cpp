// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassExecutionContext.h"
#include "MassEntityQuery.h"
#include "MassExecutor.h"
#include "Misc/SpinLock.h"
#include "TestMacros/Assertions.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

#if WITH_MASSENTITY_DEBUG
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityIterator::IndexParity", "[Mass][EntityIterator][Debug]")
{
	float NumChunksToPopulate = 2.3f;

	const int32 NumEntities = static_cast<int32>(NumChunksToPopulate * static_cast<float>(EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype)));
	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumEntities, EntitiesCreated);

	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);

	TArray<FMassEntityHandle> EntitiesIndexed;
	TArray<FMassEntityHandle> EntitiesIterated;

	Processor->ForEachEntityChunkExecutionFunction = [&EntitiesIndexed](const FMassExecutionContext& Context)
	{
		for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
		{
			EntitiesIndexed.Add(Context.GetEntity(EntityIndex));
		}
	};
	Processor->TestExecute(EntityManager);

	Processor->ForEachEntityChunkExecutionFunction = [&EntitiesIterated](FMassExecutionContext& Context)
	{
		for (FMassExecutionContext::FEntityIterator EntityIterator = Context.CreateEntityIterator(); EntityIterator; ++EntityIterator)
		{
			EntitiesIterated.Add(Context.GetEntity(EntityIterator));
		}
	};
	Processor->TestExecute(EntityManager);

	INFO("Index-based loop processes all entities");
	CHECK(EntitiesCreated.Num() == EntitiesIndexed.Num());
	INFO("Iterator-based loop processes all entities");
	CHECK(EntitiesCreated.Num() == EntitiesIterated.Num());
	INFO("Index-based and iterator-based processing produce same results");
	CHECK(FMemory::Memcmp(EntitiesIterated.GetData(), EntitiesIndexed.GetData(), EntitiesIndexed.Num()) == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityIterator::ParallelFor", "[Mass][EntityIterator][Debug]")
{
	float NumChunksToPopulate = 21.3f;

	const int32 NumEntities = static_cast<int32>(NumChunksToPopulate * static_cast<float>(EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype)));
	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumEntities, EntitiesCreated);

	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);

	TArray<FMassEntityHandle> EntitiesSync;
	TArray<FMassEntityHandle> EntitiesAsync;

	Processor->ForEachEntityChunkExecutionFunction = [&EntitiesSync](FMassExecutionContext& Context)
	{
		for (FMassExecutionContext::FEntityIterator EntityIterator = Context.CreateEntityIterator(); EntityIterator; ++EntityIterator)
		{
			EntitiesSync.Add(Context.GetEntity(EntityIterator));
		}
	};
	Processor->TestExecute(EntityManager);

	FSpinLock Lock;
	Processor->ForEachEntityChunkExecutionFunction = [&EntitiesAsync, &Lock](FMassExecutionContext& Context)
	{
		TArray<FMassEntityHandle> EntitiesAsyncLocal;
		for (FMassExecutionContext::FEntityIterator EntityIterator = Context.CreateEntityIterator(); EntityIterator; ++EntityIterator)
		{
			EntitiesAsyncLocal.Add(Context.GetEntity(EntityIterator));
		}

		Lock.Lock();
		EntitiesAsync.Append(EntitiesAsyncLocal);
		Lock.Unlock();
	};
	Processor->SetUseParallelForEachEntityChunk(true);
	Processor->TestExecute(EntityManager);

	EntitiesSync.Sort();
	EntitiesAsync.Sort();

	INFO("Index-based loop processes all entities");
	CHECK(EntitiesCreated.Num() == EntitiesSync.Num());
	INFO("Iterator-based loop processes all entities");
	CHECK(EntitiesCreated.Num() == EntitiesAsync.Num());
	INFO("Index-based and iterator-based processing produce same results");
	CHECK(FMemory::Memcmp(EntitiesAsync.GetData(), EntitiesSync.GetData(), EntitiesSync.Num()) == 0);
}
#endif // WITH_MASSENTITY_DEBUG

// Note: Queryless and Processorless both hit the same ensureMsgf callsite (MassExecutionContext.cpp:240).
// ensureMsgf fires only once per callsite per process, and Catch2 randomizes test order, so we cannot
// assert on EnsureScope.GetCount() in either test. FEnsureScope absorbs the ensure if it does fire.

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityIterator::Queryless", "[Mass][EntityIterator]")
{
	FMassExecutionContext LocalContext(*EntityManager.Get());
	{
		FEnsureScope EnsureScope;
		FMassExecutionContext::FEntityIterator FailedIterator = LocalContext.CreateEntityIterator();

		INFO("(Not) Created iterator is valid");
		CHECK_FALSE(bool(FailedIterator));

		int32 NumIterations = 0;
		for (; FailedIterator; ++FailedIterator)
		{
			++NumIterations;
		}
		INFO("Number of iterations with an invalid iterator");
		CHECK(NumIterations == 0);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityIterator::Processorless", "[Mass][EntityIterator]")
{
	FMassExecutionContext LocalContext(*EntityManager.Get());
	{
		FEnsureScope EnsureScope;
		FMassExecutionContext::FEntityIterator FailedIterator = LocalContext.CreateEntityIterator();

		INFO("(Not) Created iterator is valid");
		CHECK_FALSE(bool(FailedIterator));

		int32 NumIterations = 0;
		for (; FailedIterator; ++FailedIterator)
		{
			++NumIterations;
		}
		INFO("Number of iterations with an invalid iterator");
		CHECK(NumIterations == 0);
	}
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
