// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityManager.h"
#include "MassEntityTypes.h"
#include "MassArchetypeTypes.h"
#include "MassProcessingPhaseManager.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityLLTProcessors.h"
#include "StructUtils/InstancedStruct.h"

#include "TestHarness.h"

namespace UE::Mass::LLT
{

//-----------------------------------------------------------------------------
// FMassLLTFixture — base fixture: EntityManager only
//-----------------------------------------------------------------------------
struct FMassLLTFixture
{
	TSharedPtr<FMassEntityManager> EntityManager;

	FMassLLTFixture()
	{
		EntityManager = MakeShareable(new FMassEntityManager(GetTransientPackageAsObject()));
		EntityManager->SetDebugName(TEXT("MassEntityLLT"));

		FMassEntityManagerStorageInitParams InitializationParams;
#if WITH_MASS_CONCURRENT_RESERVE
		InitializationParams.Emplace<FMassEntityManager_InitParams_Concurrent>();
#else
		InitializationParams.Emplace<FMassEntityManager_InitParams_SingleThreaded>();
#endif
		EntityManager->Initialize(InitializationParams);
		EntityManager->PostInitialize();
	}

	~FMassLLTFixture()
	{
		EntityManager->Deinitialize();
		EntityManager = nullptr;
	}
};

//-----------------------------------------------------------------------------
// FMassLLTEntityFixture — extends base with standard archetypes
//-----------------------------------------------------------------------------
struct FMassLLTEntityFixture : FMassLLTFixture
{
	FMassArchetypeHandle EmptyArchetype;
	FMassArchetypeHandle FloatsArchetype;
	FMassArchetypeHandle IntsArchetype;
	FMassArchetypeHandle FloatsIntsArchetype;
	FMassArchetypeHandle LargeArchetype;

	FInstancedStruct InstanceInt;

	FMassLLTEntityFixture()
	{
		REQUIRE(EntityManager);

		const UScriptStruct* FragmentTypes[] = {
			FTestFragment_Float::StaticStruct(),
			FTestFragment_Int::StaticStruct(),
			FTestFragment_Large::StaticStruct()
		};

		EmptyArchetype = EntityManager->CreateArchetype(MakeArrayView<const UScriptStruct*>(nullptr, 0));
		FloatsArchetype = EntityManager->CreateArchetype(MakeArrayView(&FragmentTypes[0], 1));
		IntsArchetype = EntityManager->CreateArchetype(MakeArrayView(&FragmentTypes[1], 1));
		FloatsIntsArchetype = EntityManager->CreateArchetype(MakeArrayView(FragmentTypes, 2));
		LargeArchetype = EntityManager->CreateArchetype(MakeArrayView(&FragmentTypes[2], 1));

		FTestFragment_Int IntFrag;
		IntFrag.Value = FTestFragment_Int::TestIntValue;
		InstanceInt = FInstancedStruct::Make(IntFrag);
	}
};

//-----------------------------------------------------------------------------
// FMassLLTProcessingPhasesFixture — fixture for latent (multi-frame) tests
// Mirrors FProcessingPhasesTestBase lifecycle but as a Catch2 fixture.
//-----------------------------------------------------------------------------
struct FMassLLTProcessingPhasesFixture : FMassLLTEntityFixture
{
	TSharedPtr<FMassLLTProcessingPhaseManager> PhaseManager;
	FMassProcessingPhaseConfig PhasesConfig[static_cast<int32>(EMassProcessingPhase::MAX)];
	int32 TickIndex = -1;
	FGraphEventRef CompletionEvent;
	float DeltaTime = 1.f / 30.f;

	FMassLLTProcessingPhasesFixture()
	{
		PhaseManager = MakeShareable(new FMassLLTProcessingPhaseManager());
	}

	~FMassLLTProcessingPhasesFixture()
	{
		if (CompletionEvent.IsValid())
		{
			CompletionEvent->Wait();
		}
		PhaseManager->Stop();
		PhaseManager = nullptr;
	}

	void InitializePhaseManager()
	{
		PhaseManager->Initialize(*GetTransientPackageAsObject(), PhasesConfig);
		PhaseManager->Start(EntityManager);
	}

	void Tick()
	{
		if (CompletionEvent.IsValid())
		{
			CompletionEvent->Wait();
		}

		for (int32 PhaseIndex = 0; PhaseIndex < static_cast<int32>(EMassProcessingPhase::MAX); ++PhaseIndex)
		{
			const FGraphEventArray Prerequisites = { CompletionEvent };
			CompletionEvent = TGraphTask<FMassLLTPhaseTickTask>::CreateTask(&Prerequisites)
				.ConstructAndDispatchWhenReady(PhaseManager.ToSharedRef(), EMassProcessingPhase(PhaseIndex), DeltaTime);
		}

		++TickIndex;
	}

	void WaitForCompletion()
	{
		if (CompletionEvent.IsValid())
		{
			CompletionEvent->Wait();
		}
	}
};

} // namespace UE::Mass::LLT
