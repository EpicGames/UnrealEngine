// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTProcessors.h"
#include "MassEntityManager.h"
#include "MassExecutor.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"

UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// UMassLLTProcessorBase
//----------------------------------------------------------------------//
UMassLLTProcessorBase::UMassLLTProcessorBase()
	: EntityQuery(*this)
{
#if WITH_EDITORONLY_DATA
	bCanShowUpInSettings = false;
#endif // WITH_EDITORONLY_DATA
	bAutoRegisterWithProcessingPhases = false;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::All);

	ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context) {};

	SetUseParallelForEachEntityChunk(false);
}

void UMassLLTProcessorBase::SetUseParallelForEachEntityChunk(bool bEnable)
{
	if (bEnable)
	{
		ExecutionFunction = [this](FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
		{
			EntityQuery.ParallelForEachEntityChunk(Context, ForEachEntityChunkExecutionFunction, FMassEntityQuery::EParallelExecutionFlags::Force);
		};
	}
	else
	{
		ExecutionFunction = [this](FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
		{
			EntityQuery.ForEachEntityChunk(Context, ForEachEntityChunkExecutionFunction);
		};
	}
}

void UMassLLTProcessorBase::TestExecute(TSharedPtr<FMassEntityManager>& EntityManager)
{
	REQUIRE(EntityManager);
	UMassLLTProcessorBase* This = this;
	TArrayView<UMassProcessor* const> ProcessorView = MakeArrayView(reinterpret_cast<UMassProcessor**>(&This), 1);
	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
	UE::Mass::Executor::RunProcessorsView(ProcessorView, ProcessingContext);
}

void UMassLLTProcessorBase::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	ExecutionFunction(EntityManager, Context);
}

//----------------------------------------------------------------------//
// Typed processors
//----------------------------------------------------------------------//
void UMassLLTProcessor_Floats::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
}

void UMassLLTProcessor_Ints::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
}

void UMassLLTProcessor_FloatsInts::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
}

//----------------------------------------------------------------------//
// UMassLLTStaticCounterProcessor
//----------------------------------------------------------------------//
int32 UMassLLTStaticCounterProcessor::StaticCounter = 0;
UMassLLTStaticCounterProcessor::UMassLLTStaticCounterProcessor()
{
#if WITH_EDITORONLY_DATA
	bCanShowUpInSettings = false;
#endif // WITH_EDITORONLY_DATA
	bAutoRegisterWithProcessingPhases = false;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::All);
}

//----------------------------------------------------------------------//
// UMassLLTProcessorAutoExecuteQuery
//----------------------------------------------------------------------//
UMassLLTProcessorAutoExecuteQuery::UMassLLTProcessorAutoExecuteQuery()
{
	bAutoRegisterWithProcessingPhases = false;
}

//----------------------------------------------------------------------//
// UMassLLTProcessorAutoExecuteQueryComparison
//----------------------------------------------------------------------//
void UMassLLTProcessorAutoExecuteQueryComparison::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Large>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Array>(EMassFragmentAccess::ReadOnly);
}

void UMassLLTProcessorAutoExecuteQueryComparison::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		const TConstArrayView<FTestFragment_Float> TestFloat = Context.GetFragmentView<FTestFragment_Float>();
		const TArrayView<FTestFragment_Int> TestInt = Context.GetMutableFragmentView<FTestFragment_Int>();
		const TConstArrayView<FTestFragment_Bool> TestBool = Context.GetFragmentView<FTestFragment_Bool>();
		const TConstArrayView<FTestFragment_Large> TestLarge = Context.GetFragmentView<FTestFragment_Large>();
		const TConstArrayView<FTestFragment_Array> TestArray = Context.GetFragmentView<FTestFragment_Array>();

		for (uint32 EntityIndex : Context.CreateEntityIterator())
		{
			int32& TestIntVal = TestInt[EntityIndex].Value;
			if (TestFloat[EntityIndex].Value > 0.0f)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}

			if (TestBool[EntityIndex].bValue)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}

			if (TestLarge[EntityIndex].Value[0] > 0)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}

			if (TestArray[EntityIndex].Value.Num() > 0)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}
		}
	});
}

//----------------------------------------------------------------------//
// UMassLLTProcessorAutoExecuteQueryComparison_Parallel
//----------------------------------------------------------------------//
void UMassLLTProcessorAutoExecuteQueryComparison_Parallel::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Large>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Array>(EMassFragmentAccess::ReadOnly);
}

void UMassLLTProcessorAutoExecuteQueryComparison_Parallel::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ParallelForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		const TConstArrayView<FTestFragment_Float> TestFloat = Context.GetFragmentView<FTestFragment_Float>();
		const TArrayView<FTestFragment_Int> TestInt = Context.GetMutableFragmentView<FTestFragment_Int>();
		const TConstArrayView<FTestFragment_Bool> TestBool = Context.GetFragmentView<FTestFragment_Bool>();
		const TConstArrayView<FTestFragment_Large> TestLarge = Context.GetFragmentView<FTestFragment_Large>();
		const TConstArrayView<FTestFragment_Array> TestArray = Context.GetFragmentView<FTestFragment_Array>();

		for (uint32 EntityIndex : Context.CreateEntityIterator())
		{
			int32& TestIntVal = TestInt[EntityIndex].Value;
			if (TestFloat[EntityIndex].Value > 0.0f)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}

			if (TestBool[EntityIndex].bValue)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}

			if (TestLarge[EntityIndex].Value[0] > 0)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}

			if (TestArray[EntityIndex].Value.Num() > 0)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}
		}
	});
}

//----------------------------------------------------------------------//
// UMassLLTWorldSubsystem
//----------------------------------------------------------------------//
void UMassLLTWorldSubsystem::Write(int32 InNumber)
{
	UE_MT_SCOPED_WRITE_ACCESS(AccessDetector);
	Number = InNumber;
}

int32 UMassLLTWorldSubsystem::Read() const
{
	UE_MT_SCOPED_READ_ACCESS(AccessDetector);
	return Number;
}

//----------------------------------------------------------------------//
// FMassLLTPhaseTickTask
//----------------------------------------------------------------------//
FMassLLTPhaseTickTask::FMassLLTPhaseTickTask(const TSharedRef<FMassProcessingPhaseManager>& InPhaseManager, const EMassProcessingPhase InPhase, const float InDeltaTime)
	: PhaseManager(InPhaseManager)
	, Phase(InPhase)
	, DeltaTime(InDeltaTime)
{
}

TStatId FMassLLTPhaseTickTask::GetStatId()
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FMassLLTPhaseTickTask, STATGROUP_TaskGraphTasks);
}

ENamedThreads::Type FMassLLTPhaseTickTask::GetDesiredThread()
{
	return ENamedThreads::GameThread;
}

ESubsequentsMode::Type FMassLLTPhaseTickTask::GetSubsequentsMode()
{
	return ESubsequentsMode::TrackSubsequents;
}

void FMassLLTPhaseTickTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMassLLTPhaseTickTask);
	PhaseManager->TriggerPhase(Phase, DeltaTime, MyCompletionGraphEvent);
}

//----------------------------------------------------------------------//
// FMassLLTProcessingPhaseManager
//----------------------------------------------------------------------//
void FMassLLTProcessingPhaseManager::Start(const TSharedPtr<FMassEntityManager>& InEntityManager)
{
	EntityManager = InEntityManager;

	OnNewArchetypeHandle = EntityManager->GetOnNewArchetypeEvent().AddRaw(this, &FMassLLTProcessingPhaseManager::OnNewArchetype);

	// Deliberately skip EnableTickFunctions — phases are driven manually via FMassLLTPhaseTickTask
	bIsAllowedToTick = true;
}

void FMassLLTProcessingPhaseManager::OnNewArchetype(const FMassArchetypeHandle& NewArchetype)
{
	FMassProcessingPhaseManager::OnNewArchetype(NewArchetype);
}

UE_ENABLE_OPTIMIZATION_SHIP
