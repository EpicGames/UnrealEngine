// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassObserverProcessor.h"
#include "MassSubsystemBase.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"
#include "MassProcessingPhaseManager.h"
#include "MassEntityLLTTypes.h"

#include "TestHarness.h"

#include "MassEntityLLTProcessors.generated.h"

//-----------------------------------------------------------------------------
// UMassLLTProcessorBase — core test processor mirroring UMassTestProcessorBase
//-----------------------------------------------------------------------------
UCLASS()
class UMassLLTProcessorBase : public UMassProcessor
{
	GENERATED_BODY()
public:
	UMassLLTProcessorBase();
	FMassProcessorExecutionOrder& GetMutableExecutionOrder() { return ExecutionOrder; }
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const override { return false; }
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override {}

	using FExecutionFunction = TFunction<void(FMassEntityManager& EntityManager, FMassExecutionContext& Context)>;
	FExecutionFunction ExecutionFunction;
	FMassExecuteFunction ForEachEntityChunkExecutionFunction;

	void SetShouldAllowMultipleInstances(const bool bInShouldAllowDuplicated) { bAllowMultipleInstances = bInShouldAllowDuplicated; }
	void SetUseParallelForEachEntityChunk(bool bEnable);
	void TestExecute(TSharedPtr<FMassEntityManager>& EntityManager);

	FMassEntityCreationRequirements& GetEntityCreationRequirements() { return ProcessorEntityCreationRequirements; }

	/** public on purpose, this is a test processor */
	FMassEntityQuery EntityQuery;
};

//-----------------------------------------------------------------------------
// Named processor variants for dependency testing
//-----------------------------------------------------------------------------
UCLASS()
class UMassLLTProcessor_A : public UMassLLTProcessorBase
{
	GENERATED_BODY()
};

UCLASS()
class UMassLLTProcessor_B : public UMassLLTProcessorBase
{
	GENERATED_BODY()
};

UCLASS()
class UMassLLTProcessor_C : public UMassLLTProcessorBase
{
	GENERATED_BODY()
};

UCLASS()
class UMassLLTProcessor_D : public UMassLLTProcessorBase
{
	GENERATED_BODY()
};

UCLASS()
class UMassLLTProcessor_E : public UMassLLTProcessorBase
{
	GENERATED_BODY()
};

UCLASS()
class UMassLLTProcessor_F : public UMassLLTProcessorBase
{
	GENERATED_BODY()
};

//-----------------------------------------------------------------------------
// Typed processors — configure queries for specific fragment types
//-----------------------------------------------------------------------------
UCLASS()
class UMassLLTProcessor_Floats : public UMassLLTProcessorBase
{
	GENERATED_BODY()
public:
	TArrayView<FTestFragment_Float> Floats;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};

UCLASS()
class UMassLLTProcessor_Ints : public UMassLLTProcessorBase
{
	GENERATED_BODY()
public:
	TArrayView<FTestFragment_Int> Ints;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};

UCLASS()
class UMassLLTProcessor_FloatsInts : public UMassLLTProcessorBase
{
	GENERATED_BODY()
public:
	TArrayView<FTestFragment_Float> Floats;
	TArrayView<FTestFragment_Int> Ints;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};

//-----------------------------------------------------------------------------
// Static counter processor
//-----------------------------------------------------------------------------
UCLASS()
class UMassLLTStaticCounterProcessor : public UMassProcessor
{
	GENERATED_BODY()
public:
	UMassLLTStaticCounterProcessor();
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override
	{
		++StaticCounter;
	}
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override {}
	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const override { return false; }

	static int32 StaticCounter;
};

//-----------------------------------------------------------------------------
// Auto-execute query processors
//-----------------------------------------------------------------------------
UCLASS()
class UMassLLTProcessorAutoExecuteQuery : public UMassProcessor
{
	GENERATED_BODY()
public:
	UMassLLTProcessorAutoExecuteQuery();
	void SetAutoExecuteQuery(TSharedPtr<UE::Mass::FQueryExecutor> InAutoExecuteQuery)
	{
		AutoExecuteQuery = InAutoExecuteQuery;
	}
	FMassEntityQuery EntityQuery{ *this };
};

UCLASS()
class UMassLLTProcessorAutoExecuteQueryComparison : public UMassLLTProcessorBase
{
	GENERATED_BODY()
public:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	FMassEntityQuery EntityQuery{ *this };
};

UCLASS()
class UMassLLTProcessorAutoExecuteQueryComparison_Parallel : public UMassLLTProcessorBase
{
	GENERATED_BODY()
public:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	FMassEntityQuery EntityQuery{ *this };
};

//-----------------------------------------------------------------------------
// Composite observer processor
//-----------------------------------------------------------------------------
UCLASS()
class UMassLLTCompositeObserverProcessor : public UMassObserverProcessor
{
	GENERATED_BODY()
public:
	UMassLLTCompositeObserverProcessor()
	{
		bAutoRegisterWithObserverRegistry = false;
		QueryBasedPruning = EMassQueryBasedPruning::Never;
	}

	int32 ExecuteCallCount = 0;

	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override
	{
		++ExecuteCallCount;
	}

	void SetObservedTypesForTest(std::initializer_list<const UScriptStruct*> Types)
	{
		ObservedTypes.Reset(static_cast<int32>(Types.size()));
		for (const UScriptStruct* T : Types)
		{
			ObservedTypes.Add(T);
		}
	}

	void SetObservedOperationsForTest(const EMassObservedOperationFlags InFlags)
	{
		ObservedOperations = InFlags;
	}
};

//-----------------------------------------------------------------------------
// Subsystems
//-----------------------------------------------------------------------------
UCLASS()
class UMassLLTWorldSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()
public:
	void Write(int32 InNumber);
	int32 Read() const;

private:
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(AccessDetector);
	int32 Number = 0;
};

template<>
struct TMassExternalSubsystemTraits<UMassLLTWorldSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

UCLASS()
class UMassLLTParallelSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()
};

template<>
struct TMassExternalSubsystemTraits<UMassLLTParallelSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = true,
	};
};

UCLASS()
class UMassLLTEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
};

template<>
struct TMassExternalSubsystemTraits<UMassLLTEngineSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

UCLASS()
class UMassLLTLocalPlayerSubsystem : public ULocalPlayerSubsystem
{
	GENERATED_BODY()
};

template<>
struct TMassExternalSubsystemTraits<UMassLLTLocalPlayerSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

UCLASS()
class UMassLLTGameInstanceSubsystem : public UGameInstanceSubsystem
{
	GENERATED_BODY()
};

template<>
struct TMassExternalSubsystemTraits<UMassLLTGameInstanceSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

//-----------------------------------------------------------------------------
// Phase tick task — mirrors FMassTestPhaseTickTask for LLT
//-----------------------------------------------------------------------------
struct FMassLLTPhaseTickTask
{
	FMassLLTPhaseTickTask(const TSharedRef<FMassProcessingPhaseManager>& InPhaseManager, const EMassProcessingPhase InPhase, const float InDeltaTime);

	static TStatId GetStatId();
	static ENamedThreads::Type GetDesiredThread();
	static ESubsequentsMode::Type GetSubsequentsMode();

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

private:
	const TSharedRef<FMassProcessingPhaseManager> PhaseManager;
	const EMassProcessingPhase Phase = EMassProcessingPhase::MAX;
	const float DeltaTime = 0.f;
};

//-----------------------------------------------------------------------------
// LLT PhaseManager — mirrors FMassTestProcessingPhaseManager
// Disables tick function registration so phases are driven manually.
//-----------------------------------------------------------------------------
struct FMassLLTProcessingPhaseManager : public FMassProcessingPhaseManager
{
	void Start(const TSharedPtr<FMassEntityManager>& InEntityManager);
	void OnNewArchetype(const FMassArchetypeHandle& NewArchetype);
};

//-----------------------------------------------------------------------------
// Helper functions
//-----------------------------------------------------------------------------
template<typename T>
T* NewTestProcessor(TSharedPtr<FMassEntityManager> EntityManager)
{
	REQUIRE(EntityManager);
	T* NewProcessor = NewObject<T>();
	REQUIRE(NewProcessor);
	NewProcessor->CallInitialize(GetTransientPackageAsObject(), EntityManager.ToSharedRef());
	return NewProcessor;
}

template<typename TProcessor>
int32 SimpleProcessorRun(FMassEntityManager& EntityManager)
{
	int32 EntityProcessedCount = 0;
	TProcessor* Processor = NewObject<TProcessor>();
	Processor->CallInitialize(GetTransientPackageAsObject(), EntityManager.AsShared());
	Processor->ForEachEntityChunkExecutionFunction = [&EntityProcessedCount](FMassExecutionContext& Context)
	{
		EntityProcessedCount += Context.GetNumEntities();
	};

	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
	UE::Mass::Executor::Run(*Processor, ProcessingContext);

	return EntityProcessedCount;
}
