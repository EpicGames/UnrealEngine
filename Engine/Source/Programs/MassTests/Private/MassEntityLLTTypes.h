// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "Mass/EntityHandle.h"
#include "MassEntityLLTTypes.generated.h"

//-----------------------------------------------------------------------------
// Fragments
//-----------------------------------------------------------------------------
USTRUCT()
struct FTestFragment_Float : public FMassFragment
{
	GENERATED_BODY()
	float Value = 0;

	FTestFragment_Float(const float InValue = 0.f) : Value(InValue) {}
};

USTRUCT()
struct FTestFragment_Int : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;

	FTestFragment_Int(const int32 InValue = 0) : Value(InValue) {}

	static constexpr int32 TestIntValue = 123456;
};

USTRUCT()
struct FTestFragment_Bool : public FMassFragment
{
	GENERATED_BODY()
	bool bValue = false;

	FTestFragment_Bool(const bool bInValue = false) : bValue(bInValue) {}
};

USTRUCT()
struct FTestFragment_Large : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	uint8 Value[64];

	FTestFragment_Large(uint8 Fill = 0)
	{
		FMemory::Memset(Value, Fill, 64);
	}
};

USTRUCT()
struct FTestFragment_Array : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<int32> Value;

	FTestFragment_Array(uint8 Num = 0)
	{
		Value.Reserve(Num);
	}
};

template<>
struct TMassFragmentTraits<FTestFragment_Array> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

USTRUCT()
struct FFragmentWithSharedPtr : public FMassFragment
{
	GENERATED_BODY()

	TSharedPtr<int32> Data;

	FFragmentWithSharedPtr() = default;
	FFragmentWithSharedPtr(TSharedPtr<int32>& InData)
		: Data(InData)
	{}
};

template<>
struct TMassFragmentTraits<FFragmentWithSharedPtr> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

//-----------------------------------------------------------------------------
// Chunk Fragments
//-----------------------------------------------------------------------------
USTRUCT()
struct FTestChunkFragment_Int : public FMassChunkFragment
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;

	FTestChunkFragment_Int(const int32 InValue = 0) : Value(InValue) {}
};

USTRUCT()
struct FTestChunkFragment_Float : public FMassChunkFragment
{
	GENERATED_BODY()

	UPROPERTY()
	float Value = 0.f;

	FTestChunkFragment_Float(const float InValue = 0.f) : Value(InValue) {}
};

//-----------------------------------------------------------------------------
// Shared Fragments
//-----------------------------------------------------------------------------
USTRUCT()
struct FTestSharedFragment_Int : public FMassSharedFragment
{
	using FValueType = int32;

	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;

	FTestSharedFragment_Int(const int32 InValue = 0) : Value(InValue) {}
};

USTRUCT()
struct FTestSharedFragment_Array : public FMassSharedFragment
{
	using FValueType = TArray<int32>;

	GENERATED_BODY()

	UPROPERTY()
	TArray<int32> Value;

	FTestSharedFragment_Array() = default;
	FTestSharedFragment_Array(TArray<int32>&& InValue) : Value(MoveTemp(InValue)) {}
};

template<>
struct TMassSharedFragmentTraits<FTestSharedFragment_Int> final
{
	enum
	{
		GameThreadOnly = true
	};
};

USTRUCT()
struct FTestConstSharedFragment_Int : public FMassConstSharedFragment
{
	using FValueType = int32;

	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;

	FTestConstSharedFragment_Int(const int32 InValue = 0) : Value(InValue) {}
};

USTRUCT()
struct FTestSharedFragment_Float : public FMassSharedFragment
{
	using FValueType = float;

	GENERATED_BODY()

	UPROPERTY()
	float Value = 0.f;

	FTestSharedFragment_Float(const float InValue = 0) : Value(InValue) {}
};

USTRUCT()
struct FTestConstSharedFragment_Float : public FMassConstSharedFragment
{
	using FValueType = float;

	GENERATED_BODY()

	UPROPERTY()
	float Value = 0.f;

	FTestConstSharedFragment_Float(const float InValue = 0) : Value(InValue) {}
};

//-----------------------------------------------------------------------------
// Tags
//-----------------------------------------------------------------------------
USTRUCT()
struct FTestFragment_Tag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_A : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_B : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_C : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_D : public FMassTag
{
	GENERATED_BODY()
};

//-----------------------------------------------------------------------------
// Sparse Fragments & Tags
//-----------------------------------------------------------------------------
USTRUCT()
struct FTestFragment_SparseInt : public FMassSparseFragment
{
	GENERATED_BODY()

	FTestFragment_SparseInt(const int32 InValue = 0)
		: Value(InValue)
	{
	}

	int32 Value = 0;
};

USTRUCT()
struct FTestFragment_SparseFloat : public FMassSparseFragment
{
	GENERATED_BODY()

	FTestFragment_SparseFloat(const float InValue = 0.f)
		: Value(InValue)
	{
	}

	float Value = 0.f;
};

USTRUCT()
struct FTestTag_SparseA : public FMassSparseTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_SparseB : public FMassSparseTag
{
	GENERATED_BODY()
};

//-----------------------------------------------------------------------------
// Entity Handle Fragment (for indirect access tests)
//-----------------------------------------------------------------------------
USTRUCT()
struct FTestFragment_EntityHandle : public FMassFragment
{
	GENERATED_BODY()

	FMassEntityHandle Value;

	FTestFragment_EntityHandle(const FMassEntityHandle InValue = FMassEntityHandle())
		: Value(InValue)
	{
		
	}
};
