// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"

#if WITH_MASS_CONCURRENT_RESERVE


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

enum class EEntityManagerMode : int32
{
	Sequential,
	Concurrent
};

//-----------------------------------------------------------------------------
// Helper fixture for storage parity tests
//-----------------------------------------------------------------------------
struct FEntityStorageFixture
{
	TArray<TSharedPtr<FMassEntityManager>> EntityManagers;
	TArray<EEntityManagerMode> ManagersToCreate;
	const uint32 MaxConcurrentEntitiesPerPage = 0;
	const int32 TotalNumToReserve = 0;

	enum EEntityOperation
	{
		Operation_IndividualReserve,
		Operation_BatchReserve,
		Operation_IndividualDestroy,
		Operation_BatchDestroy,
		OperationsCount,
	};
	const int32 OperationsNumLimit = OperationsCount * 5;
	int32 NumToReserveInOneIteration = 0;
	int32 NumToReleaseInOneIteration = 0;
	FRandomStream RandomStream;

	FEntityStorageFixture()
		: ManagersToCreate({ EEntityManagerMode::Sequential, EEntityManagerMode::Concurrent })
		, MaxConcurrentEntitiesPerPage(FMassEntityManager_InitParams_Concurrent().MaxEntitiesPerPage)
		, TotalNumToReserve(MaxConcurrentEntitiesPerPage * 3 / 2)
		, RandomStream(1)
	{
		NumToReserveInOneIteration = TotalNumToReserve / 10;
		NumToReleaseInOneIteration = TotalNumToReserve / 12;
		Initialize();
	}

	explicit FEntityStorageFixture(TArray<EEntityManagerMode> InModes)
		: ManagersToCreate(MoveTemp(InModes))
		, MaxConcurrentEntitiesPerPage(FMassEntityManager_InitParams_Concurrent().MaxEntitiesPerPage)
		, TotalNumToReserve(MaxConcurrentEntitiesPerPage * 3 / 2)
		, RandomStream(1)
	{
		NumToReserveInOneIteration = TotalNumToReserve / 10;
		NumToReleaseInOneIteration = TotalNumToReserve / 12;
		Initialize();
	}

	void Initialize()
	{
		int32 Index = 0;
		for (const EEntityManagerMode Mode : ManagersToCreate)
		{
			TSharedPtr<FMassEntityManager> LocalEntityManager = MakeShareable(new FMassEntityManager());
			LocalEntityManager->SetDebugName(FString::Printf(TEXT("TestEntityManager_%d"), Index));
			++Index;

			FMassEntityManagerStorageInitParams InitializationParams;
			if (Mode == EEntityManagerMode::Sequential)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				InitializationParams.Emplace<FMassEntityManager_InitParams_SingleThreaded>();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			else
			{
				InitializationParams.Emplace<FMassEntityManager_InitParams_Concurrent>();
				InitializationParams.Get<FMassEntityManager_InitParams_Concurrent>().MaxEntitiesPerPage = MaxConcurrentEntitiesPerPage;
			}
			LocalEntityManager->Initialize(InitializationParams);
			EntityManagers.Add(LocalEntityManager);
		}
	}

	~FEntityStorageFixture()
	{
		for (TSharedPtr<FMassEntityManager>& LocalEntityManager : EntityManagers)
		{
			LocalEntityManager->Deinitialize();
		}
	}

	void PerformOperation(const EEntityOperation CurrentOperation, TSharedPtr<FMassEntityManager>& EntityManager, TArray<FMassEntityHandle>& EntitiesReserved)
	{
		switch (CurrentOperation)
		{
		case Operation_IndividualReserve:
			for (int32 Counter = 0; Counter < NumToReserveInOneIteration; ++Counter)
			{
				EntitiesReserved.Add(EntityManager->ReserveEntity());
			}
			break;
		case Operation_BatchReserve:
			EntityManager->BatchReserveEntities(NumToReserveInOneIteration, EntitiesReserved);
			break;
		case Operation_IndividualDestroy:
		{
			const int32 NumToRelease = FMath::Min(NumToReleaseInOneIteration, EntitiesReserved.Num());
			for (int32 Counter = 0; Counter < NumToRelease; ++Counter)
			{
				const int32 Idx = RandomStream.RandRange(0, EntitiesReserved.Num() - 1);
				EntityManager->ReleaseReservedEntity(EntitiesReserved[Idx]);
				EntitiesReserved.RemoveAtSwap(Idx, EAllowShrinking::No);
			}
		}
		break;
		case Operation_BatchDestroy:
		{
			const int32 NumToRelease = FMath::Min(NumToReleaseInOneIteration, EntitiesReserved.Num());
			TArray<FMassEntityHandle> EntitiesToDestroy;
			EntitiesToDestroy.Reserve(NumToRelease);
			for (int32 Counter = 0; Counter < NumToRelease; ++Counter)
			{
				const int32 Idx = RandomStream.RandRange(0, EntitiesReserved.Num() - 1);
				EntitiesToDestroy.Add(EntitiesReserved[Idx]);
				EntitiesReserved.RemoveAtSwap(Idx, EAllowShrinking::No);
			}
			EntityManager->BatchDestroyEntities(EntitiesToDestroy);
		}
		break;
		}
	}

	bool ValidateUniqueAndValidEntities(TSharedPtr<FMassEntityManager>& EntityManager, TConstArrayView<FMassEntityHandle> ContainerToTest) const
	{
		TArray<FMassEntityHandle> Sorted;
		Sorted.Append(ContainerToTest.GetData(), ContainerToTest.Num());
		Sorted.Sort();
		for (int32 Index = Sorted.Num() - 1; Index >= 0; --Index)
		{
			if (Index > 0 && Sorted[Index].Index == Sorted[Index - 1].Index)
			{
				return false;
			}
			if (!EntityManager->IsEntityValid(Sorted[Index]))
			{
				return false;
			}
			if (EntityManager->IsEntityBuilt(Sorted[Index]))
			{
				return false;
			}
		}
		return true;
	}
};

TEST_CASE("Mass::Storage::SingleEntityParity", "[Mass][Storage]")
{
	FEntityStorageFixture Fixture;
	FMassEntityHandle ReservedEntitySequential = Fixture.EntityManagers[static_cast<int32>(EEntityManagerMode::Sequential)]->ReserveEntity();
	FMassEntityHandle ReservedEntityConcurrent = Fixture.EntityManagers[static_cast<int32>(EEntityManagerMode::Concurrent)]->ReserveEntity();
	CHECK(ReservedEntitySequential == ReservedEntityConcurrent);
}

TEST_CASE("Mass::Storage::Sequential::BatchParity", "[Mass][Storage][Sequential]")
{
	FEntityStorageFixture Fixture({ EEntityManagerMode::Sequential, EEntityManagerMode::Sequential });
	const int32 NumToReserve = Fixture.MaxConcurrentEntitiesPerPage * 3 / 2;

	TArray<FMassEntityHandle> EntitiesIndividual;
	for (int32 Index = 0; Index < NumToReserve; ++Index)
	{
		EntitiesIndividual.Add(Fixture.EntityManagers[0]->ReserveEntity());
	}
	CHECK(EntitiesIndividual.Num() == NumToReserve);

	TArray<FMassEntityHandle> EntitiesBatch;
	Fixture.EntityManagers[1]->BatchReserveEntities(NumToReserve, EntitiesBatch);
	CHECK(EntitiesBatch.Num() == NumToReserve);

	for (int32 Index = 0; Index < NumToReserve; ++Index)
	{
		CHECK(EntitiesIndividual[Index].Index == EntitiesBatch[Index].Index);
	}
}

TEST_CASE("Mass::Storage::Concurrent::BatchParity", "[Mass][Storage][Concurrent]")
{
	FEntityStorageFixture Fixture({ EEntityManagerMode::Concurrent, EEntityManagerMode::Concurrent });
	const int32 NumToReserve = Fixture.MaxConcurrentEntitiesPerPage * 3 / 2;

	TArray<FMassEntityHandle> EntitiesIndividual;
	for (int32 Index = 0; Index < NumToReserve; ++Index)
	{
		EntitiesIndividual.Add(Fixture.EntityManagers[0]->ReserveEntity());
	}
	CHECK(EntitiesIndividual.Num() == NumToReserve);

	TArray<FMassEntityHandle> EntitiesBatch;
	Fixture.EntityManagers[1]->BatchReserveEntities(NumToReserve, EntitiesBatch);
	CHECK(EntitiesBatch.Num() == NumToReserve);

	for (int32 Index = 0; Index < NumToReserve; ++Index)
	{
		CHECK(EntitiesIndividual[Index].Index == EntitiesBatch[Index].Index);
	}
}

TEST_CASE("Mass::Storage::Sequential::FreeList", "[Mass][Storage][Sequential]")
{
	FEntityStorageFixture Fixture({ EEntityManagerMode::Sequential, EEntityManagerMode::Sequential, EEntityManagerMode::Sequential });
	const int32 NumToReserve = Fixture.MaxConcurrentEntitiesPerPage * 3 / 2;

	TArray<FMassEntityHandle> EntitiesBaseline;
	Fixture.EntityManagers[0]->BatchReserveEntities(NumToReserve, EntitiesBaseline);

	// batch-reserving and batch-removing
	TArray<FMassEntityHandle> EntitiesTestedBatch;
	{
		Fixture.EntityManagers[1]->BatchReserveEntities(NumToReserve, EntitiesTestedBatch);
		TArray<FMassEntityHandle> EntitiesToModify;
		EntitiesToModify.Reserve(EntitiesTestedBatch.Num() / 2);
		for (int32 Index = EntitiesTestedBatch.Num() - 1; Index >= 0; --Index)
		{
			if (EntitiesTestedBatch[Index].Index % 2)
			{
				EntitiesToModify.Add(EntitiesTestedBatch[Index]);
				EntitiesTestedBatch.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}
		const int32 EntitiesRemovedCount = EntitiesToModify.Num();
		Fixture.EntityManagers[1]->BatchDestroyEntities(EntitiesToModify);
		EntitiesToModify.Reset();
		Fixture.EntityManagers[1]->BatchReserveEntities(EntitiesRemovedCount, EntitiesToModify);
		CHECK(EntitiesToModify.Num() == EntitiesRemovedCount);
		EntitiesTestedBatch.Append(EntitiesToModify);
		CHECK(EntitiesTestedBatch.Num() == EntitiesBaseline.Num());
	}

	// batch-reserving and removing one-by-one
	TArray<FMassEntityHandle> EntitiesTestedIndividual;
	{
		Fixture.EntityManagers[2]->BatchReserveEntities(NumToReserve, EntitiesTestedIndividual);
		for (int32 Index = EntitiesTestedIndividual.Num() - 1; Index >= 0; --Index)
		{
			if (Fixture.RandomStream.FRand() >= 0.5)
			{
				Fixture.EntityManagers[2]->ReleaseReservedEntity(EntitiesTestedIndividual[Index]);
				EntitiesTestedIndividual.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}
		Fixture.EntityManagers[2]->BatchReserveEntities(NumToReserve - EntitiesTestedIndividual.Num(), EntitiesTestedIndividual);
		CHECK(EntitiesTestedIndividual.Num() == NumToReserve);
	}

	EntitiesTestedIndividual.Sort();
	EntitiesTestedBatch.Sort();
	EntitiesBaseline.Sort();
	for (int32 Index = 0; Index < EntitiesTestedBatch.Num(); ++Index)
	{
		CHECK(EntitiesTestedBatch[Index].Index == EntitiesBaseline[Index].Index);
		CHECK(EntitiesTestedIndividual[Index].Index == EntitiesBaseline[Index].Index);
	}
}

TEST_CASE("Mass::Storage::Concurrent::FreeList", "[Mass][Storage][Concurrent]")
{
	FEntityStorageFixture Fixture({ EEntityManagerMode::Concurrent, EEntityManagerMode::Concurrent, EEntityManagerMode::Concurrent });
	const int32 NumToReserve = Fixture.MaxConcurrentEntitiesPerPage * 3 / 2;

	TArray<FMassEntityHandle> EntitiesBaseline;
	Fixture.EntityManagers[0]->BatchReserveEntities(NumToReserve, EntitiesBaseline);

	TArray<FMassEntityHandle> EntitiesTestedBatch;
	{
		Fixture.EntityManagers[1]->BatchReserveEntities(NumToReserve, EntitiesTestedBatch);
		TArray<FMassEntityHandle> EntitiesToModify;
		EntitiesToModify.Reserve(EntitiesTestedBatch.Num() / 2);
		for (int32 Index = EntitiesTestedBatch.Num() - 1; Index >= 0; --Index)
		{
			if (EntitiesTestedBatch[Index].Index % 2)
			{
				EntitiesToModify.Add(EntitiesTestedBatch[Index]);
				EntitiesTestedBatch.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}
		const int32 EntitiesRemovedCount = EntitiesToModify.Num();
		Fixture.EntityManagers[1]->BatchDestroyEntities(EntitiesToModify);
		EntitiesToModify.Reset();
		Fixture.EntityManagers[1]->BatchReserveEntities(EntitiesRemovedCount, EntitiesToModify);
		CHECK(EntitiesToModify.Num() == EntitiesRemovedCount);
		EntitiesTestedBatch.Append(EntitiesToModify);
		CHECK(EntitiesTestedBatch.Num() == EntitiesBaseline.Num());
	}

	TArray<FMassEntityHandle> EntitiesTestedIndividual;
	{
		Fixture.EntityManagers[2]->BatchReserveEntities(NumToReserve, EntitiesTestedIndividual);
		for (int32 Index = EntitiesTestedIndividual.Num() - 1; Index >= 0; --Index)
		{
			if (Fixture.RandomStream.FRand() >= 0.5)
			{
				Fixture.EntityManagers[2]->ReleaseReservedEntity(EntitiesTestedIndividual[Index]);
				EntitiesTestedIndividual.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}
		Fixture.EntityManagers[2]->BatchReserveEntities(NumToReserve - EntitiesTestedIndividual.Num(), EntitiesTestedIndividual);
		CHECK(EntitiesTestedIndividual.Num() == NumToReserve);
	}

	EntitiesTestedIndividual.Sort();
	EntitiesTestedBatch.Sort();
	EntitiesBaseline.Sort();
	for (int32 Index = 0; Index < EntitiesTestedBatch.Num(); ++Index)
	{
		CHECK(EntitiesTestedBatch[Index].Index == EntitiesBaseline[Index].Index);
		CHECK(EntitiesTestedIndividual[Index].Index == EntitiesBaseline[Index].Index);
	}
}

TEST_CASE("Mass::Storage::Sequential::AddRemoveLoop", "[Mass][Storage][Sequential]")
{
	FEntityStorageFixture Fixture({ EEntityManagerMode::Sequential });
	TSharedPtr<FMassEntityManager> EM = Fixture.EntityManagers[0];
	TArray<FMassEntityHandle> EntitiesReserved;

	Fixture.PerformOperation(FEntityStorageFixture::Operation_BatchReserve, EM, EntitiesReserved);
	Fixture.PerformOperation(FEntityStorageFixture::Operation_BatchDestroy, EM, EntitiesReserved);
	Fixture.PerformOperation(FEntityStorageFixture::Operation_BatchReserve, EM, EntitiesReserved);
	CHECK(Fixture.ValidateUniqueAndValidEntities(EM, EntitiesReserved));

	Fixture.PerformOperation(FEntityStorageFixture::Operation_BatchReserve, EM, EntitiesReserved);
	CHECK(Fixture.ValidateUniqueAndValidEntities(EM, EntitiesReserved));
}

TEST_CASE("Mass::Storage::Concurrent::AddRemoveLoop", "[Mass][Storage][Concurrent]")
{
	FEntityStorageFixture Fixture({ EEntityManagerMode::Concurrent });
	TSharedPtr<FMassEntityManager> EM = Fixture.EntityManagers[0];
	TArray<FMassEntityHandle> EntitiesReserved;

	Fixture.PerformOperation(FEntityStorageFixture::Operation_BatchReserve, EM, EntitiesReserved);
	Fixture.PerformOperation(FEntityStorageFixture::Operation_BatchDestroy, EM, EntitiesReserved);
	Fixture.PerformOperation(FEntityStorageFixture::Operation_BatchReserve, EM, EntitiesReserved);
	CHECK(Fixture.ValidateUniqueAndValidEntities(EM, EntitiesReserved));

	Fixture.PerformOperation(FEntityStorageFixture::Operation_BatchReserve, EM, EntitiesReserved);
	CHECK(Fixture.ValidateUniqueAndValidEntities(EM, EntitiesReserved));
}

TEST_CASE("Mass::Storage::Sequential::MixedOperations", "[Mass][Storage][Sequential]")
{
	FEntityStorageFixture Fixture({ EEntityManagerMode::Sequential });
	TSharedPtr<FMassEntityManager> EM = Fixture.EntityManagers[0];

	TArray<FMassEntityHandle> EntitiesReserved;
	FEntityStorageFixture::EEntityOperation CurrentOperation = FEntityStorageFixture::Operation_BatchReserve;
	int32 OpsCount = 0;
	while (EntitiesReserved.Num() < Fixture.TotalNumToReserve && OpsCount < Fixture.OperationsNumLimit)
	{
		Fixture.PerformOperation(CurrentOperation, EM, EntitiesReserved);
		CHECK(Fixture.ValidateUniqueAndValidEntities(EM, EntitiesReserved));
		++OpsCount;
		CurrentOperation = FEntityStorageFixture::EEntityOperation(Fixture.RandomStream.RandRange(0, 3));
	}
}

TEST_CASE("Mass::Storage::Concurrent::MixedOperations", "[Mass][Storage][Concurrent]")
{
	FEntityStorageFixture Fixture({ EEntityManagerMode::Concurrent });
	TSharedPtr<FMassEntityManager> EM = Fixture.EntityManagers[0];

	TArray<FMassEntityHandle> EntitiesReserved;
	FEntityStorageFixture::EEntityOperation CurrentOperation = FEntityStorageFixture::Operation_BatchReserve;
	int32 OpsCount = 0;
	while (EntitiesReserved.Num() < Fixture.TotalNumToReserve && OpsCount < Fixture.OperationsNumLimit)
	{
		Fixture.PerformOperation(CurrentOperation, EM, EntitiesReserved);
		CHECK(Fixture.ValidateUniqueAndValidEntities(EM, EntitiesReserved));
		++OpsCount;
		CurrentOperation = FEntityStorageFixture::EEntityOperation(Fixture.RandomStream.RandRange(0, 3));
	}
}

#if WITH_MASSENTITY_DEBUG
TEST_CASE("Mass::Storage::Concurrent::DataLayoutAssumptions", "[Mass][Storage][Concurrent][Debug]")
{
	const bool bAssumptionsValid = UE::Mass::FConcurrentEntityStorage::DebugAssumptionsSelfTest();
	CHECK(bAssumptionsValid);
}
#endif // WITH_MASSENTITY_DEBUG

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP

#endif // WITH_MASS_CONCURRENT_RESERVE
