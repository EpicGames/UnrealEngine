// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityBuilder.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

#if WITH_MASSENTITY_DEBUG
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::ArchetypeCreation", "[Mass][Entity][Debug]")
{
	CHECK(FloatsArchetype.IsValid());
	CHECK(IntsArchetype.IsValid());

	TArray<const UScriptStruct*> FragmentsList;
	EntityManager->DebugGetArchetypeFragmentTypes(FloatsArchetype, FragmentsList);
	REQUIRE(FragmentsList.Num() == 1);
	CHECK(FragmentsList[0] == FTestFragment_Float::StaticStruct());

	FragmentsList.Reset();
	EntityManager->DebugGetArchetypeFragmentTypes(IntsArchetype, FragmentsList);
	REQUIRE(FragmentsList.Num() == 1);
	CHECK(FragmentsList[0] == FTestFragment_Int::StaticStruct());

	FragmentsList.Reset();
	EntityManager->DebugGetArchetypeFragmentTypes(FloatsIntsArchetype, FragmentsList);
	CHECK(FragmentsList.Num() == 2);
	CHECK((FragmentsList.Find(FTestFragment_Int::StaticStruct()) != INDEX_NONE && FragmentsList.Find(FTestFragment_Float::StaticStruct()) != INDEX_NONE));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::ArchetypeEquivalence", "[Mass][Entity][Debug]")
{
	TArray<const UScriptStruct*> FragmentsA = { FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct() };
	TArray<const UScriptStruct*> FragmentsB = { FTestFragment_Int::StaticStruct(), FTestFragment_Float::StaticStruct() };
	const FMassArchetypeHandle ArchetypeA = EntityManager->CreateArchetype(FragmentsA);
	const FMassArchetypeHandle ArchetypeB = EntityManager->CreateArchetype(FragmentsB);

	INFO("Archetype creation is expected to be independent of fragments ordering");
	CHECK(ArchetypeA == ArchetypeB);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::MultipleEntitiesCreation", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	int32 Counts[] = { 10, 100, 1000 };
	int32 TotalCreatedCount = 0;
	FMassArchetypeHandle Archetypes[] = { FloatsArchetype, IntsArchetype, FloatsIntsArchetype };

	for (int32 ArchetypeIndex = 0; ArchetypeIndex < (sizeof(Archetypes) / sizeof(FMassArchetypeHandle)); ++ArchetypeIndex)
	{
		for (int32 EntityIndex = 0; EntityIndex < Counts[ArchetypeIndex]; ++EntityIndex)
		{
			EntityManager->CreateEntity(Archetypes[ArchetypeIndex]);
		}
		TotalCreatedCount += Counts[ArchetypeIndex];
	}
	CHECK(EntityManager->DebugGetEntityCount() == TotalCreatedCount);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 10);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 100);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == 1000);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::BatchCreation", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const int32 Count = 123;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);
	CHECK(Entities.Num() == Count);
	CHECK(EntityManager->DebugGetEntityCount() == Count);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::BatchCreatingSingleEntity", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, /*Count=*/1, Entities);
	CHECK(Entities.Num() == 1);
	CHECK(EntityManager->DebugGetEntityCount() == 1);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::EntityCreation", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsArchetype);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 1);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) + EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::EntityCreationFromInstances", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle Entity = EntityManager->CreateEntity(MakeArrayView(&InstanceInt, 1));
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 1);
	CHECK(EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity).Value == FTestFragment_Int::TestIntValue);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::AddingFragmentType", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
	EntityManager->AddFragmentToEntity(Entity, FTestFragment_Int::StaticStruct());
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == 1);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 0);
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsIntsArchetype);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::AddingFragmentInstance", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
	EntityManager->AddFragmentInstanceListToEntity(Entity, MakeArrayView(&InstanceInt, 1));
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == 1);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 0);
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsIntsArchetype);
	CHECK(EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity).Value == FTestFragment_Int::TestIntValue);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::RemovingFragment", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsIntsArchetype);
	EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_Float::StaticStruct());
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == 0);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 1);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::RemovingLastFragment", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
	EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_Float::StaticStruct());
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == EmptyArchetype);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::DestroyEntity", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsArchetype);
	EntityManager->DestroyEntity(Entity);
	CHECK(EntityManager->DebugGetEntityCount() == 0);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::EntityReservationAndBuilding", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle ReservedEntity = EntityManager->ReserveEntity();
	CHECK(EntityManager->IsEntityValid(ReservedEntity));
	CHECK_FALSE(EntityManager->IsEntityBuilt(ReservedEntity));
	EntityManager->BuildEntity(ReservedEntity, FloatsArchetype);
	CHECK(EntityManager->IsEntityBuilt(ReservedEntity));
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->GetArchetypeForEntity(ReservedEntity) == FloatsArchetype);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 1);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) + EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == 0);
	EntityManager->DestroyEntity(ReservedEntity);
	CHECK(EntityManager->DebugGetEntityCount() == 0);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::EntityReservationAndBuildingFromInstances", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle ReservedEntity = EntityManager->ReserveEntity();
	CHECK(EntityManager->IsEntityValid(ReservedEntity));
	CHECK_FALSE(EntityManager->IsEntityBuilt(ReservedEntity));
	EntityManager->BuildEntity(ReservedEntity, MakeArrayView(&InstanceInt, 1));
	CHECK(EntityManager->IsEntityBuilt(ReservedEntity));
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->GetArchetypeForEntity(ReservedEntity) == IntsArchetype);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 1);
	CHECK(EntityManager->GetFragmentDataChecked<FTestFragment_Int>(ReservedEntity).Value == FTestFragment_Int::TestIntValue);
	EntityManager->DestroyEntity(ReservedEntity);
	CHECK(EntityManager->DebugGetEntityCount() == 0);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::ReleaseEntity", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	const FMassEntityHandle ReservedEntity = EntityManager->ReserveEntity();
	CHECK(EntityManager->IsEntityValid(ReservedEntity));
	CHECK_FALSE(EntityManager->IsEntityBuilt(ReservedEntity));
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->GetArchetypeForEntity(ReservedEntity) == FMassArchetypeHandle());
	EntityManager->ReleaseReservedEntity(ReservedEntity);
	CHECK(EntityManager->DebugGetEntityCount() == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::ReserveAPreviouslyBuiltEntity", "[Mass][Entity][Debug]")
{
	REQUIRE(EntityManager);
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
		CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);
		EntityManager->DestroyEntity(Entity);
		CHECK(EntityManager->DebugGetEntityCount() == 0);
		CHECK(EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 0);
	}

	const FMassEntityHandle ReservedEntity = EntityManager->ReserveEntity();
	CHECK(EntityManager->IsEntityValid(ReservedEntity));
	CHECK_FALSE(EntityManager->IsEntityBuilt(ReservedEntity));
	CHECK(EntityManager->DebugGetEntityCount() == 1);
	CHECK(EntityManager->GetArchetypeForEntity(ReservedEntity) == FMassArchetypeHandle());
	EntityManager->BuildEntity(ReservedEntity, FloatsArchetype);
	CHECK(EntityManager->IsEntityBuilt(ReservedEntity));
	CHECK(EntityManager->GetArchetypeForEntity(ReservedEntity) == FloatsArchetype);
	EntityManager->DestroyEntity(ReservedEntity);
	CHECK(EntityManager->DebugGetEntityCount() == 0);
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);
}

#endif // WITH_MASSENTITY_DEBUG

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::SharedPtrFragment", "[Mass][Entity]")
{
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, Entities);

	TArray<TWeakPtr<int32>> SharedPtrs;
	for (int32 Index = 0; Index < Entities.Num(); ++Index)
	{
		TSharedPtr<int32> TestData = MakeShared<int32>(Index);
		SharedPtrs.Add(TestData.ToWeakPtr());

		TArray<FInstancedStruct> Array;
		Array.AddZeroed();
		Array[0].InitializeAs<FFragmentWithSharedPtr>(TestData);

		const FMassEntityHandle& EntityHandle = Entities[Index];
		EntityManager->AddFragmentInstanceListToEntity(EntityHandle, Array);
	}

	for (int32 Index = 0; Index < Entities.Num(); ++Index)
	{
		TSharedPtr<int32> TestData = SharedPtrs[Index].Pin();
		const FMassEntityHandle& EntityHandle = Entities[Index];
		const FFragmentWithSharedPtr& Fragment = EntityManager->GetFragmentDataChecked<FFragmentWithSharedPtr>(EntityHandle);
		CHECK(*Fragment.Data == *TestData);
	}

	EntityManager->AddTagToEntity(Entities[0], FTestTag_A::StaticStruct());
	EntityManager->AddFragmentToEntity(Entities[1], FTestFragment_Float::StaticStruct());

	for (int32 Index = 0; Index < Entities.Num(); ++Index)
	{
		TSharedPtr<int32> TestData = SharedPtrs[Index].Pin();
		const FMassEntityHandle& EntityHandle = Entities[Index];
		const FFragmentWithSharedPtr& Fragment = EntityManager->GetFragmentDataChecked<FFragmentWithSharedPtr>(EntityHandle);
		CHECK(*Fragment.Data == *TestData);
	}

	EntityManager->BatchDestroyEntities(Entities);
	for (int32 Index = 0; Index < Entities.Num(); ++Index)
	{
		TSharedPtr<int32> TestData = SharedPtrs[Index].Pin();
		CHECK_FALSE(TestData.IsValid());
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::CreationPatterns", "[Mass][Entity]")
{
	TArray<FMassEntityHandle> Entities;

	// create entities straight from fragment instances
	TArray<FInstancedStruct> FragmentInstances = {
		FInstancedStruct::Make<FTestFragment_Int>()
		, FInstancedStruct::Make<FTestFragment_Float>()
	};
	Entities.Add(EntityManager->CreateEntity(FragmentInstances));

	// create the archetype first, and then create an entity within it
	TArray<const UScriptStruct*> FragmentTypes = {
		FTestFragment_Int::StaticStruct()
		, FTestFragment_Float::StaticStruct()
	};
	FMassArchetypeHandle ArchetypeHandle = EntityManager->CreateArchetype(FragmentTypes);
	Entities.Add(EntityManager->CreateEntity(ArchetypeHandle));

	// Reserve + Build
	Entities.Add(EntityManager->ReserveEntity());
	EntityManager->BuildEntity(Entities.Last(), ArchetypeHandle);

	// entity builder
	Entities.Add(EntityManager->MakeEntityBuilder().SetForceDeferredCommit(false)
		.Add<FTestFragment_Int>(1024)
		.Add<FTestFragment_Float>(3.14f)
		.Commit());

	for (FMassEntityHandle EntityHandle : Entities)
	{
		const FMassArchetypeHandle ResultArchetypeHandle = EntityManager->GetArchetypeForEntity(EntityHandle);
		CHECK(ResultArchetypeHandle == FloatsIntsArchetype);
	}
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
