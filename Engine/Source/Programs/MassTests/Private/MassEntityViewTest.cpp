// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityView.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::EntityView::Invalidate", "[Mass][EntityView]")
{
	const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(IntsArchetype);
	FMassEntityView EntityView(*EntityManager.Get(), EntityHandle);

	CHECK(EntityView.IsValid());

	EntityManager->AddTagToEntity(EntityHandle, FTestTag_A::StaticStruct());

	CHECK_FALSE(EntityView.IsValid());
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
