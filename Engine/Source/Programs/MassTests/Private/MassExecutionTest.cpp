// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityManager.h"
#include "MassProcessingContext.h"
#include "MassExecutor.h"
#include "Logging/LogScopedVerbosityOverride.h"

#include "TestHarness.h"
#include "TestMacros/Assertions.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Execution::Setup", "[Mass][Execution]")
{
	INFO("EntitySubsystem needs to be created for the test to be performed");
	CHECK(EntityManager.Get() != nullptr);
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Execution::EmptyArray", "[Mass][Execution]")
{
	REQUIRE(EntityManager);
	const float DeltaSeconds = 0.f;
	FMassProcessingContext ProcessingContext(*EntityManager, DeltaSeconds);
	// no test performed, let's just see if it results in errors/warnings
	UE::Mass::Executor::RunProcessorsView(TArrayView<UMassProcessor*>(), ProcessingContext);
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Execution::EmptyPipeline", "[Mass][Execution]")
{
	REQUIRE(EntityManager);
	const float DeltaSeconds = 0.f;
	FMassProcessingContext ProcessingContext(*EntityManager, DeltaSeconds);
	FMassRuntimePipeline Pipeline;
	// no test performed, let's just see if it results in errors/warnings
	UE::Mass::Executor::Run(Pipeline, ProcessingContext);
}

#if WITH_MASSENTITY_DEBUG
TEST_CASE_METHOD(FMassLLTFixture, "Mass::Execution::SingleNullProcessor", "[Mass][Execution][Debug]")
{
	REQUIRE(EntityManager);
	const float DeltaSeconds = 0.f;
	FMassProcessingContext ProcessingContext(EntityManager, DeltaSeconds);
	TArray<UMassProcessor*> Processors;
	Processors.Add(nullptr);

	// Production code logs UE_LOG(LogMass, Error) for nullptr processors — suppress to avoid noisy output
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogMass, ELogVerbosity::Fatal);
	UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Execution::SingleValidProcessor", "[Mass][Execution][Debug]")
{
	REQUIRE(EntityManager);
	const float DeltaSeconds = 0.f;
	FMassProcessingContext ProcessingContext(EntityManager, DeltaSeconds);
	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	REQUIRE(Processor);
	// need to set up some requirements to make EntityQuery valid
	Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

	// nothing should break. The actual result of processing is getting tested in MassProcessorTest.cpp
	UE::Mass::Executor::Run(*Processor, ProcessingContext);
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Execution::MultipleNullProcessors", "[Mass][Execution][Debug]")
{
	REQUIRE(EntityManager);
	const float DeltaSeconds = 0.f;
	FMassProcessingContext ProcessingContext(EntityManager, DeltaSeconds);
	TArray<UMassProcessor*> Processors;
	Processors.Add(nullptr);
	Processors.Add(nullptr);
	Processors.Add(nullptr);

	// Production code logs UE_LOG(LogMass, Error) for nullptr processors — suppress to avoid noisy output
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogMass, ELogVerbosity::Fatal);
	UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);
}
#endif // WITH_MASSENTITY_DEBUG

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Execution::Sparse", "[Mass][Execution]")
{
	REQUIRE(EntityManager);
	const float DeltaSeconds = 0.f;
	FMassProcessingContext ProcessingContext(*EntityManager, DeltaSeconds);
	UMassLLTProcessorBase* Processor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	REQUIRE(Processor);
	// need to set up some requirements to make EntityQuery valid
	Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

	FMassRuntimePipeline Pipeline;
	{
		TArray<UMassProcessor*> Processors;
		Processors.Add(Processor);
		Pipeline.SetProcessors(Processors);
	}

	FMassArchetypeEntityCollection EntityCollection(FloatsArchetype);
	// nothing should break. The actual result of processing is getting tested in MassProcessorTest.cpp

	UE::Mass::Executor::RunWithCollection(Pipeline, ProcessingContext, EntityCollection);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
