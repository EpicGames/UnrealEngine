// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityManager.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "MassExecutor.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

template<typename T>
static FName GetProcessorName()
{
	return T::StaticClass()->GetFName();
}

//----------------------------------------------------------------------//
// Multithreading.Trivial — single processor, TaskGraph-based parallel exec
//----------------------------------------------------------------------//
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Multithreading::Trivial", "[Mass][Multithreading]")
{
	const int32 NumToCreate = 200;
	int32 NumProcessed = 0;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumToCreate, Entities);

	TArray<UMassLLTProcessorBase*> Processors;
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Proc->ForEachEntityChunkExecutionFunction = [&NumProcessed](FMassExecutionContext& Context)
		{
			NumProcessed += Context.GetNumEntities();
		};
	}

	UMassCompositeProcessor* CompositeProcessor = NewObject<UMassCompositeProcessor>();
	CompositeProcessor->SetGroupName(TEXT("Test"));
	CompositeProcessor->SetProcessors(MakeArrayView<UMassProcessor*>((UMassProcessor**)Processors.GetData(), Processors.Num()));

	FMassProcessingContext Context(*EntityManager);
	FGraphEventRef FinishEvent = UE::Mass::Executor::TriggerParallelTasks(*CompositeProcessor, MoveTemp(Context), []() {});
	FinishEvent->Wait();

	INFO("Expected to process all the created entities.");
	CHECK(NumToCreate == NumProcessed);
}

//----------------------------------------------------------------------//
// Multithreading.Basic — 3 processors with ordering dependencies,
// verify data flows correctly through the pipeline.
// A sets Int=index, B sets Float=Int*Int, C sets Int=Float+Int
// Expected: Int = i*i + i
//----------------------------------------------------------------------//
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Multithreading::Basic", "[Mass][Multithreading]")
{
	const int32 NumToCreate = 200;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumToCreate, Entities);

	TArray<UMassLLTProcessorBase*> Processors;

	// Processor C: reads Float + reads/writes Int; executes after B
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_C>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassLLTProcessor_B>());
		Proc->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		Proc->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context)
		{
			const TArrayView<FTestFragment_Int> IntsList = Context.GetMutableFragmentView<FTestFragment_Int>();
			const TConstArrayView<FTestFragment_Float> FloatsList = Context.GetFragmentView<FTestFragment_Float>();
			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				IntsList[EntityIndex].Value = static_cast<int32>(FloatsList[EntityIndex].Value) + IntsList[EntityIndex].Value;
			}
		};
	}

	// Processor B: reads Int, writes Float; executes after A
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_B>(EntityManager));
		Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassLLTProcessor_A>());
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Proc->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
		Proc->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context)
		{
			const TConstArrayView<FTestFragment_Int> IntsList = Context.GetFragmentView<FTestFragment_Int>();
			const TArrayView<FTestFragment_Float> FloatsList = Context.GetMutableFragmentView<FTestFragment_Float>();
			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				FloatsList[EntityIndex].Value = static_cast<float>(IntsList[EntityIndex].Value * IntsList[EntityIndex].Value);
			}
		};
	}

	// Processor A: writes Int = index
	{
		UMassLLTProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassLLTProcessor_A>(EntityManager));
		Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		Proc->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context)
		{
			int32 Index = 0;
			const TArrayView<FTestFragment_Int> IntsList = Context.GetMutableFragmentView<FTestFragment_Int>();
			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				IntsList[EntityIndex].Value = Index++;
			}
		};
	}

	UMassCompositeProcessor* CompositeProcessor = NewObject<UMassCompositeProcessor>();
	CompositeProcessor->SetGroupName(TEXT("Test"));
	CompositeProcessor->SetProcessors(MakeArrayView<UMassProcessor*>((UMassProcessor**)Processors.GetData(), Processors.Num()));

	FMassProcessingContext ProcessingContext(*EntityManager);
	FGraphEventRef FinishEvent = UE::Mass::Executor::TriggerParallelTasks(*CompositeProcessor, MoveTemp(ProcessingContext), []() {});
	FinishEvent->Wait();

	for (int32 EntityIndex = 0; EntityIndex < Entities.Num(); ++EntityIndex)
	{
		FMassEntityView View(FloatsIntsArchetype, Entities[EntityIndex]);
		INFO("Should have predicted values");
		CHECK(View.GetFragmentData<FTestFragment_Int>().Value == EntityIndex * EntityIndex + EntityIndex);
	}
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
