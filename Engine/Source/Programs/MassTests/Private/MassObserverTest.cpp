// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"

#include "MassEntityManager.h"
#include "MassProcessingTypes.h"
#include "MassEntityTypes.h"
#include "MassCommandBuffer.h"
#include "MassExecutionContext.h"
#include "MassObserverManager.h"
#include "MassObserverNotificationTypes.h"

#include "TestHarness.h"
#include "TestMacros/Assertions.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

auto EntityIndexSorted = [](const FMassEntityHandle& A, const FMassEntityHandle& B)
{
	return A.Index < B.Index;
};

//----------------------------------------------------------------------//
// Tag observer tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Tag::SingleEntitySingleArchetypeAdd", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[1] };
	EntityManager->Defer().AddTag<FTagStruct>(EntitiesInt[1]);

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Tag::SingleEntitySingleArchetypeRemove", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[1] };

	EntityManager->Defer().AddTag<FTagStruct>(EntitiesInt[1]);
	EntityManager->FlushCommands();
	INFO("Tag addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	EntityManager->Defer().RemoveTag<FTagStruct>(EntitiesInt[1]);

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Tag::SingleEntitySingleArchetypeDestroy", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[1] };
	EntityManager->Defer().AddTag<FTagStruct>(EntitiesInt[1]);
	EntityManager->FlushCommands();
	INFO("FTagStruct addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	EntityManager->Defer().DestroyEntity(EntitiesInt[1]);

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Tag::MultipleArchetypesAdd", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
	}

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Tag::MultipleArchetypesAdd_Sync", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->AddTagToEntity(ModifiedEntity, FTagStruct::StaticStruct());
	}

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Tag::MultipleArchetypesRemove", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
	}
	EntityManager->FlushCommands();
	INFO("FTagStruct addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().RemoveTag<FTagStruct>(ModifiedEntity);
	}

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Tag::MultipleArchetypesRemove_Sync", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->AddTagToEntity(ModifiedEntity, FTagStruct::StaticStruct());
	}

	INFO("FTagStruct addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->RemoveTagFromEntity(ModifiedEntity, FTagStruct::StaticStruct());
	}

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Tag::MultipleArchetypesDestroy", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
	}
	EntityManager->FlushCommands();
	INFO("Tag addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().DestroyEntity(ModifiedEntity);
	}

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::ForbidModifyOnDestroy", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());

		// try changing the input entities' composition.
		const bool bIsProcessing = Context.GetEntityManagerChecked().IsProcessing();
		for (const FMassEntityHandle EntityHandle : Context.GetEntities())
		{
			// intentionally empty - testing that the observer runs without crashes
		}

		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
	}
	EntityManager->FlushCommands();

	INFO("Tag addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().DestroyEntity(ModifiedEntity);
	}

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Tag::MultipleArchetypesSwap", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesIntsFloat[1], EntitiesInt[0], EntitiesInt[2] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
	}
	EntityManager->FlushCommands();
	INFO("Tag addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().SwapTags<FTagStruct, FTestTag_B>(ModifiedEntity);
	}

	EntityManager->FlushCommands();
	INFO("The observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Create::TagIndividualEntities", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	constexpr int32 EntitiesToSpawnCount = 6;

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	int32 ArrayMidPoint = 0;
	{
		TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnCount, EntitiesInt);
		ArrayMidPoint = EntitiesInt.Num() / 2;

		for (int32 Index = 0; Index < ArrayMidPoint; ++Index)
		{
			EntityManager->AddTagToEntity(EntitiesInt[Index], FTagStruct::StaticStruct());
		}
		INFO("The tag observer is not expected to run yet");
		CHECK(AffectedEntities.Num() == 0);
	}
	INFO("The tag observer is expected to run just after FEntityCreationContext's destruction");
	CHECK(AffectedEntities.Num() == ArrayMidPoint);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Create::TagBatchedEntities", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	constexpr int32 EntitiesToSpawnCount = 6;

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	{
		TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnCount, EntitiesInt);

		EntityManager->BatchChangeTagsForEntities(CreationContext->GetEntityCollections(*EntityManager.Get()), FMassTagBitSet(FTagStruct::StaticStruct()), FMassTagBitSet());
		INFO("The tag observer is not expected to run yet");
		CHECK(AffectedEntities.Num() == 0);
		INFO("CreationContext's entity collection should be invalidated at this moment");
		CHECK_FALSE(CreationContext->DebugAreEntityCollectionsUpToDate());

		EntityManager->BatchChangeTagsForEntities(CreationContext->GetEntityCollections(*EntityManager.Get()), FMassTagBitSet(FTagStruct::StaticStruct()), FMassTagBitSet());
		INFO("The tag observer is still not expected to run");
		CHECK(AffectedEntities.Num() == 0);
	}
	INFO("The tag observer is expected to run just after FEntityCreationContext's destruction");
	CHECK(AffectedEntities.Num() > 0);
	INFO("The tag observer is expected to process every entity just once");
	CHECK(AffectedEntities.Num() == EntitiesInt.Num());
}

//----------------------------------------------------------------------//
// Fragment observer tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Fragment::SingleEntitySingleArchetypeAdd", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[1] };
	EntityManager->Defer().AddFragment<FFragmentStruct>(EntitiesInt[1]);

	EntityManager->FlushCommands();
	INFO("The fragment observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Fragment::SingleEntitySingleArchetypeRemove", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[1] };

	EntityManager->Defer().AddFragment<FFragmentStruct>(EntitiesInt[1]);
	EntityManager->FlushCommands();
	INFO("Fragment addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	EntityManager->Defer().RemoveFragment<FFragmentStruct>(EntitiesInt[1]);

	EntityManager->FlushCommands();
	INFO("The fragment observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Fragment::SingleEntitySingleArchetypeDestroy", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[1] };
	EntityManager->Defer().AddFragment<FFragmentStruct>(EntitiesInt[1]);
	EntityManager->FlushCommands();
	INFO("Fragment addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	EntityManager->Defer().DestroyEntity(EntitiesInt[1]);

	EntityManager->FlushCommands();
	INFO("The fragment observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Fragment::MultipleArchetypesAdd", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesInt[1] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().AddFragment<FFragmentStruct>(ModifiedEntity);
	}
	// also adding the fragment to the other archetype that already has the fragment. This should not yield any results
	for (const FMassEntityHandle& OtherEntity : EntitiesIntsFloat)
	{
		EntityManager->Defer().AddFragment<FFragmentStruct>(OtherEntity);
	}

	EntityManager->FlushCommands();
	INFO("The fragment observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Fragment::MultipleArchetypesRemove", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().AddFragment<FFragmentStruct>(ModifiedEntity);
	}
	EntityManager->FlushCommands();
	INFO("Fragment addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().RemoveFragment<FFragmentStruct>(ModifiedEntity);
	}

	EntityManager->FlushCommands();
	INFO("The fragment observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Fragment::MultipleArchetypesDestroy", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Remove, ObserverProcessor);

	TArray<FMassEntityHandle> ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().AddFragment<FFragmentStruct>(ModifiedEntity);
	}
	EntityManager->FlushCommands();
	INFO("Fragment addition is not being observed and is not expected to produce results yet");
	CHECK(AffectedEntities.Num() == 0);
	for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
	{
		EntityManager->Defer().DestroyEntity(ModifiedEntity);
	}

	EntityManager->FlushCommands();
	INFO("The fragment observer is expected to be run for predicted number of entities");
	REQUIRE(AffectedEntities.Num() == ExpectedEntities.Num());
	INFO("The commands issued by the observer are flushed");
	CHECK(bCommandsFlushed);

	ExpectedEntities.Sort(EntityIndexSorted);
	AffectedEntities.Sort(EntityIndexSorted);

	for (int32 EntityIndex = 0; EntityIndex < ExpectedEntities.Num(); ++EntityIndex)
	{
		INFO("Expected and affected sets should be the same");
		CHECK(AffectedEntities[EntityIndex] == ExpectedEntities[EntityIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Create::FragmentSingleEntity", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);

	constexpr float TestValue = 123.456f;
	float ValueOnNotification = 0.f;

	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&ValueOnNotification](FMassExecutionContext& Context)
		{
			const TConstArrayView<FFragmentStruct> Fragments = Context.GetFragmentView<FFragmentStruct>();
			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); EntityIndex++)
			{
				ValueOnNotification = Fragments[EntityIndex].Value;
			};
		};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FInstancedStruct> FragmentInstanceList = { FInstancedStruct::Make(FFragmentStruct(TestValue)) };

	// BuildEntity
	{
		const FMassEntityHandle Entity = EntityManager->ReserveEntity();
		EntityManager->BuildEntity(Entity, FragmentInstanceList);
		INFO("The fragment observer notified by BuildEntity is expected to be able to fetch the initial value");
		CHECK(ValueOnNotification == TestValue);
		EntityManager->DestroyEntity(Entity);
	}

	// CreateEntity
	{
		ValueOnNotification = 0.f;
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FragmentInstanceList);
		INFO("The fragment observer notified by CreateEntity is expected to be able to fetch the initial value");
		CHECK(ValueOnNotification == TestValue);
		EntityManager->DestroyEntity(Entity);
	}

	ObserverProcessor->ForEachEntityChunkExecutionFunction = nullptr;
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Create::FragmentIndividualEntities", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	constexpr int32 EntitiesToSpawnCount = 6;

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	int32 ArrayMidPoint = 0;
	{
		TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnCount, EntitiesInt);
		ArrayMidPoint = EntitiesInt.Num() / 2;

		for (int32 Index = 0; Index < ArrayMidPoint; ++Index)
		{
			EntityManager->AddFragmentToEntity(EntitiesInt[Index], FFragmentStruct::StaticStruct());
		}
		INFO("The fragment observer is not expected to run yet");
		CHECK(AffectedEntities.Num() == 0);
	}
	INFO("The fragment observer is expected to run just after FEntityCreationContext's destruction");
	CHECK(AffectedEntities.Num() == ArrayMidPoint);
}

#if WITH_MASSENTITY_DEBUG
// @todo Requires WITH_AITESTSUITE for testableCheckf to short-circuit CHECK_SYNC_API().
// Without it, checkf fires but AddFragmentToEntity still executes, causing wrong entity counts.
DISABLED_TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::ChangingCompositionSync", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	constexpr int32 EntitiesToSpawn = 3;
	const FMassArchetypeHandle OriginalArchetype = FloatsArchetype;

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesInt;
	{
		FCheckScope CheckScope;

		ObserverProcessor->ForEachEntityChunkExecutionFunction = [&EntityManager = EntityManager](FMassExecutionContext& Context)
		{
			EntityManager->AddFragmentToEntity(Context.GetEntity(0), FTestFragment_Int::StaticStruct());
		};

		EntityManager->BatchCreateEntities(OriginalArchetype, EntitiesToSpawn, EntitiesInt);
		CHECK(CheckScope.GetCount() > 0);

		INFO("Number of entities in the original archetype, no moves expected");
		CHECK(EntityManager->DebugGetArchetypeEntitiesCount(OriginalArchetype) == EntitiesToSpawn);
	}
	{
		FCheckScope CheckScope;

		ObserverProcessor->ForEachEntityChunkExecutionFunction = [&EntityManager = EntityManager, OriginalArchetype](FMassExecutionContext& Context)
		{
			FMassArchetypeEntityCollection EntityCollection(OriginalArchetype, Context.GetEntities(), FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchChangeFragmentCompositionForEntities(MakeArrayView(&EntityCollection, 1)
				, FMassFragmentBitSet(FTestFragment_Int::StaticStruct()), {});
		};

		EntityManager->BatchCreateEntities(OriginalArchetype, EntitiesToSpawn, EntitiesInt);
		CHECK(CheckScope.GetCount() > 0);

		INFO("Number of entities in the original archetype, no moves expected");
		CHECK(EntityManager->DebugGetArchetypeEntitiesCount(OriginalArchetype) == EntitiesToSpawn * 2);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::ChangingCompositionDeferred", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);

	ObserverProcessor->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context)
		{
			Context.Defer().PushCommand<FMassCommandAddFragments<FTestFragment_Int>>(Context.GetEntities());
		};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	constexpr int32 EntitiesToSpawn = 3;
	const FMassArchetypeHandle OriginalArchetype = FloatsArchetype;

	TArray<FMassEntityHandle> EntitiesInt;
	EntityManager->BatchCreateEntities(OriginalArchetype, EntitiesToSpawn, EntitiesInt);

	INFO("Number of entities in the original archetype");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(OriginalArchetype) == 0);
	INFO("Number of entities in the target archetype");
	CHECK(EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype) == EntitiesToSpawn);
}
#endif // WITH_MASSENTITY_DEBUG

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Create::ModificationsToOtherEntities", "[Mass][Observer]")
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	constexpr int32 EntitiesToSpawnInFirstBatch = 3;
	constexpr int32 EntitiesToSpawnInSecondBatch = 5;

	TArray<FMassEntityHandle> EntitiesInt;
	EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnInFirstBatch, EntitiesInt);
	FMassArchetypeEntityCollection InitialEntitiesCollection(IntsArchetype, EntitiesInt, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	{
		TSharedRef<FMassObserverManager::FObserverLock> ObserversLock = EntityManager->GetOrMakeObserversLock();
		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnInSecondBatch, EntitiesInt);
			ensure(EntitiesInt.Num() == EntitiesToSpawnInFirstBatch + EntitiesToSpawnInSecondBatch);
		}
		EntityManager->BatchChangeTagsForEntities(MakeArrayView(&InitialEntitiesCollection, 1)
			, FMassTagBitSet(FTagStruct::StaticStruct()), FMassTagBitSet());

		INFO("The tag observer is not expected to run yet");
		CHECK(AffectedEntities.Num() == 0);
	}
	INFO("The tag observer is expected to run just after FEntityCreationContext's destruction");
	CHECK(AffectedEntities.Num() > 0);
	INFO("The tag observer is expected to process only the original entities, that had a tag added to them");
	CHECK(AffectedEntities.Num() == EntitiesToSpawnInFirstBatch);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Create::CreationOperationOrder::Batch", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;
	int32 Counter = 0;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);

	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&Counter](FMassExecutionContext& Context)
		{
			for (auto EntityId : Context.CreateEntityIterator())
			{
				Context.GetMutableFragmentView<FFragmentStruct>()[EntityId].Value = static_cast<float>(++Counter);
			}
		};

	TArray<FMassEntityHandle> PreExistingEntities;
	EntityManager->BatchCreateEntities(IntsArchetype, 2, PreExistingEntities);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> NewEntities;

	// BuildScenario - Batch variant
	{
		TSharedRef<FMassObserverManager::FObserverLock> ObserversLock = EntityManager->GetOrMakeObserversLock();
		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(FloatsArchetype, 1, NewEntities);
			EntityManager->BatchCreateEntities(FloatsArchetype, 1, NewEntities);
		}
		{
			FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&PreExistingEntities[0], 1), FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchChangeFragmentCompositionForEntities(MakeArrayView(&Collection, 1), FMassFragmentBitSet(FTestFragment_Float::StaticStruct()), {});
		}
		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(FloatsArchetype, 1, NewEntities);
			EntityManager->BatchCreateEntities(FloatsArchetype, 1, NewEntities);
		}
		{
			FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&PreExistingEntities[1], 1), FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchChangeFragmentCompositionForEntities(MakeArrayView(&Collection, 1), FMassFragmentBitSet(FTestFragment_Float::StaticStruct()), {});
		}
	}

	float FirstBatchValues[] = {
		EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[0]).Value
		, EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[1]).Value
	};
	INFO("First batch's values match");
	CHECK(((FirstBatchValues[0] == 1.f && FirstBatchValues[1] == 2.f) || (FirstBatchValues[1] == 1.f && FirstBatchValues[0] == 2.f)));

	float PreExistingEntitiesValues[] = {
		EntityManager->GetFragmentDataChecked<FTestFragment_Float>(PreExistingEntities[0]).Value
		, EntityManager->GetFragmentDataChecked<FTestFragment_Float>(PreExistingEntities[1]).Value
	};
	INFO("First preexisting entity's value");
	CHECK(PreExistingEntitiesValues[0] == 3.f);
	INFO("Second preexisting entity's value");
	CHECK(PreExistingEntitiesValues[1] == 6.f);

	float SecondBatchValues[] = {
		EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[2]).Value
		, EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[3]).Value
	};
	INFO("Second batch's values match");
	CHECK(((SecondBatchValues[0] == 4.f && SecondBatchValues[1] == 5.f) || (SecondBatchValues[1] == 4.f && SecondBatchValues[0] == 5.f)));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Create::CreationOperationOrder::Individual", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;
	int32 Counter = 0;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);

	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&Counter](FMassExecutionContext& Context)
		{
			for (auto EntityId : Context.CreateEntityIterator())
			{
				Context.GetMutableFragmentView<FFragmentStruct>()[EntityId].Value = static_cast<float>(++Counter);
			}
		};

	TArray<FMassEntityHandle> PreExistingEntities;
	EntityManager->BatchCreateEntities(IntsArchetype, 2, PreExistingEntities);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> NewEntities;

	// BuildScenario - Individual variant
	{
		TSharedRef<FMassObserverManager::FObserverLock> ObserversLock = EntityManager->GetOrMakeObserversLock();
		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->GetOrMakeCreationContext();
			NewEntities.Add(EntityManager->CreateEntity(FloatsArchetype));
			NewEntities.Add(EntityManager->CreateEntity(FloatsArchetype));
		}
		EntityManager->AddFragmentToEntity(PreExistingEntities[0], FTestFragment_Float::StaticStruct());
		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->GetOrMakeCreationContext();
			NewEntities.Add(EntityManager->CreateEntity(FloatsArchetype));
			NewEntities.Add(EntityManager->CreateEntity(FloatsArchetype));
		}
		EntityManager->AddFragmentToEntity(PreExistingEntities[1], FTestFragment_Float::StaticStruct());
	}

	float FirstBatchValues[] = {
		EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[0]).Value
		, EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[1]).Value
	};
	INFO("First batch's values match");
	CHECK(((FirstBatchValues[0] == 1.f && FirstBatchValues[1] == 2.f) || (FirstBatchValues[1] == 1.f && FirstBatchValues[0] == 2.f)));

	float PreExistingEntitiesValues[] = {
		EntityManager->GetFragmentDataChecked<FTestFragment_Float>(PreExistingEntities[0]).Value
		, EntityManager->GetFragmentDataChecked<FTestFragment_Float>(PreExistingEntities[1]).Value
	};
	INFO("First preexisting entity's value");
	CHECK(PreExistingEntitiesValues[0] == 3.f);
	INFO("Second preexisting entity's value");
	CHECK(PreExistingEntitiesValues[1] == 6.f);

	float SecondBatchValues[] = {
		EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[2]).Value
		, EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[3]).Value
	};
	INFO("Second batch's values match");
	CHECK(((SecondBatchValues[0] == 4.f && SecondBatchValues[1] == 5.f) || (SecondBatchValues[1] == 4.f && SecondBatchValues[0] == 5.f)));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::Create::UnobservedCompositionChange", "[Mass][Observer]")
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	bool bCommandsFlushed = false;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
	ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bCommandsFlushed, &AffectedEntities](FMassExecutionContext& Context)
	{
		AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushed](FMassEntityManager&)
			{
				bCommandsFlushed = true;
			});
	};

	constexpr int32 EntitiesToSpawnCount = 6;

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	TArray<FMassEntityHandle> EntitiesFloats;
	{
		TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesToSpawnCount, EntitiesFloats);

		// add unobserved tag
		EntityManager->BatchChangeTagsForEntities(CreationContext->GetEntityCollections(*EntityManager.Get()), FMassTagBitSet(FTestTag_A::StaticStruct()), FMassTagBitSet());

		INFO("The fragment observer is not expected to run yet");
		CHECK(AffectedEntities.Num() == 0);
	}
	INFO("The fragment observer is expected to run just after FEntityCreationContext's destruction");
	CHECK(AffectedEntities.Num() == EntitiesToSpawnCount);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::MoveToAnotherArchetype", "[Mass][Observer]")
{
	bool bTagAdded = false;
	bool bTagRemoved = false;
	bool bFloatAdded = false;
	bool bFloatRemoved = false;

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();

	auto CreateObserver = [this](const TFunction<void(FMassExecutionContext& Context)>& StoreResultFunction)
	{
		UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
		ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		ObserverProcessor->ForEachEntityChunkExecutionFunction = StoreResultFunction;
		return ObserverProcessor;
	};

	ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Add
		, CreateObserver([&bTagAdded](FMassExecutionContext& Context)
			{
				bTagAdded = true;
			}
		));

	ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Remove
		, CreateObserver([&bTagRemoved](FMassExecutionContext& Context)
			{
				bTagRemoved = true;
			}
		));

	ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Add
		, CreateObserver([&bFloatAdded](FMassExecutionContext& Context)
			{
				bFloatAdded = true;
			}
		));

	ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Remove
		, CreateObserver([&bFloatRemoved](FMassExecutionContext& Context)
			{
				bFloatRemoved = true;
			}
		));

	const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(IntsArchetype);

	// create target archetype
	const FMassArchetypeHandle TargetArchetypeHandle = EntityManager->CreateArchetype(IntsArchetype, { FTestTag_A::StaticStruct(), FTestFragment_Float::StaticStruct() });

	EntityManager->MoveEntityToAnotherArchetype(EntityHandle, TargetArchetypeHandle);
	INFO("Tag addition observer has been executed");
	CHECK(bTagAdded);
	INFO("Fragment addition observer has been executed");
	CHECK(bFloatAdded);
	INFO("(NOT) Tag removal observer has been executed");
	CHECK_FALSE(bTagRemoved);
	INFO("(NOT) Fragment removal observer has been executed");
	CHECK_FALSE(bFloatRemoved);

	// moving back to the original archetype will remove the two added elements, and should trigger observers
	EntityManager->MoveEntityToAnotherArchetype(EntityHandle, IntsArchetype);
	INFO("Tag removal observer has been executed");
	CHECK(bTagRemoved);
	INFO("Fragment removal observer has been executed");
	CHECK(bFloatRemoved);
}

//----------------------------------------------------------------------//
// Multi-type observer processor tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::CompositeObserver::DeduplicationTwoTypes", "[Mass][Observer]")
{
	int32 ExecuteCallCount = 0;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->ExecutionFunction = [&ExecuteCallCount](FMassEntityManager&, FMassExecutionContext&)
	{
		++ExecuteCallCount;
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	// Register the same processor instance for both Float and Int addition.
	ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);
	ObserverManager.AddObserverInstance(FTestFragment_Int::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	// Moving EmptyArchetype -> FloatsIntsArchetype adds both Float and Int simultaneously.
	const FMassEntityHandle Entity = EntityManager->CreateEntity(EmptyArchetype);
	EntityManager->MoveEntityToAnotherArchetype(Entity, FloatsIntsArchetype);

	INFO("Observer Execute should fire exactly once even though two observed types were added simultaneously");
	CHECK(ExecuteCallCount == 1);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::CompositeObserver::PartialTypeMatch", "[Mass][Observer]")
{
	int32 ExecuteCallCount = 0;

	UMassLLTProcessorBase* ObserverProcessor = NewTestProcessor<UMassLLTProcessorBase>(EntityManager);
	ObserverProcessor->ExecutionFunction = [&ExecuteCallCount](FMassEntityManager&, FMassExecutionContext&)
	{
		++ExecuteCallCount;
	};

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);
	ObserverManager.AddObserverInstance(FTestFragment_Int::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

	// Entity starts in IntsArchetype (already has Int). Moving to FloatsIntsArchetype only adds Float,
	// so only the Float pipeline is visited in HandleElementsImpl.
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, 1, Entities);
	INFO("Observer Execute should fire exactly once, for the initial type");
	CHECK(ExecuteCallCount == 1);
	ExecuteCallCount = 0;

	EntityManager->MoveEntityToAnotherArchetype(Entities[0], FloatsIntsArchetype);

	INFO("Observer Execute should fire exactly once, for the single new type");
	CHECK(ExecuteCallCount == 1);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::CompositeObserver::BaseApiAddObserver", "[Mass][Observer]")
{
	UMassLLTCompositeObserverProcessor* CompositeProcessor = NewTestProcessor<UMassLLTCompositeObserverProcessor>(EntityManager);
	CompositeProcessor->SetObservedTypesForTest({ FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct() });
	CompositeProcessor->SetObservedOperationsForTest(EMassObservedOperationFlags::Add);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	// Register via the generalized base class overload.
	ObserverManager.AddObserverInstance(CompositeProcessor);

	// Adding both Float and Int simultaneously should trigger exactly one Execute.
	const FMassEntityHandle Entity = EntityManager->CreateEntity(EmptyArchetype);
	EntityManager->MoveEntityToAnotherArchetype(Entity, FloatsIntsArchetype);

	INFO("Composite observer registered via base API should execute exactly once");
	CHECK(CompositeProcessor->ExecuteCallCount == 1);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::CompositeObserver::BaseApiRemoveObserver", "[Mass][Observer]")
{
	UMassLLTCompositeObserverProcessor* CompositeProcessor = NewTestProcessor<UMassLLTCompositeObserverProcessor>(EntityManager);
	CompositeProcessor->SetObservedTypesForTest({ FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct() });
	CompositeProcessor->SetObservedOperationsForTest(EMassObservedOperationFlags::Add);

	FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
	ObserverManager.AddObserverInstance(CompositeProcessor);

	// First move: observer should fire.
	const FMassEntityHandle Entity1 = EntityManager->CreateEntity(EmptyArchetype);
	EntityManager->MoveEntityToAnotherArchetype(Entity1, FloatsIntsArchetype);
	INFO("Observer should fire once before removal");
	CHECK(CompositeProcessor->ExecuteCallCount == 1);

	// Remove from all type pipelines via the base API.
	ObserverManager.RemoveObserverInstance(CompositeProcessor);

	// Second move: observer should NOT fire.
	const FMassEntityHandle Entity2 = EntityManager->CreateEntity(EmptyArchetype);
	EntityManager->MoveEntityToAnotherArchetype(Entity2, FloatsIntsArchetype);
	INFO("Observer should not fire after removal");
	CHECK(CompositeProcessor->ExecuteCallCount == 1);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Observer::CompositeObserver::GetObservedTypes", "[Mass][Observer]")
{
	UMassLLTCompositeObserverProcessor* Proc = NewTestProcessor<UMassLLTCompositeObserverProcessor>(EntityManager);

	INFO("Freshly created composite observer should have no observed types");
	CHECK(Proc->GetObservedTypes().Num() == 0);

	Proc->SetObservedTypesForTest({ FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct() });

	INFO("Composite observer should report 3 observed types");
	REQUIRE(Proc->GetObservedTypes().Num() == 3);
	INFO("First observed type should be Float");
	CHECK(Proc->GetObservedTypes()[0] == FTestFragment_Float::StaticStruct());
	INFO("Second observed type should be Int");
	CHECK(Proc->GetObservedTypes()[1] == FTestFragment_Int::StaticStruct());
	INFO("Third observed type should be Tag_A");
	CHECK(Proc->GetObservedTypes()[2] == FTestTag_A::StaticStruct());
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
