// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassExecutionContext.h"
#include "MassArchetypeData.h"
#include "MassCommands.h"

#include <atomic>


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ParallelForBasic", "[Mass][Query][Parallel]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();

	const int32 NumEntitiesToCreate = 32 * EntitiesPerChunk;
	TArray<FMassEntityHandle> CreatedEntities;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumEntitiesToCreate, CreatedEntities);

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);

	std::atomic<int32> EntitiesProcessed = 0;
	std::atomic<int32> ConcurrentAccess = 0;
	FCriticalSection ParallelDetectionCS;

	FMassExecutionContext Context = EntityManager->CreateExecutionContext(/*DeltaSeconds=*/0.f);
	Query.ParallelForEachEntityChunk(Context, [&ParallelDetectionCS, &EntitiesProcessed, &ConcurrentAccess](FMassExecutionContext& Context)
		{
			EntitiesProcessed += Context.GetNumEntities();

			if (ParallelDetectionCS.TryLock() == false)
			{
				++ConcurrentAccess;
				return;
			}
			FPlatformProcess::Sleep(0.01f);
		}, FMassEntityQuery::EParallelExecutionFlags::Force);

	CHECK(EntitiesProcessed.load() == NumEntitiesToCreate);
	CHECK(ConcurrentAccess.load() > 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ParallelForCollection", "[Mass][Query][Parallel]")
{
	REQUIRE(EntityManager);

	const int32 NumEntitiesToCreate = 1000;
	TArray<FMassEntityHandle> CreatedEntities;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumEntitiesToCreate, CreatedEntities);

	TArray<FMassEntityHandle> EntitiesToProcess;
	EntitiesToProcess.Reserve(NumEntitiesToCreate);

	FRandomStream RandomStream(/*Seed=*/1);
	for (FMassEntityHandle& EntityHandle : CreatedEntities)
	{
		if (RandomStream.FRand() < 0.5)
		{
			EntitiesToProcess.Add(EntityHandle);
		}
	}
	CHECK(EntitiesToProcess.Num() > 0);
	const FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesToProcess, FMassArchetypeEntityCollection::NoDuplicates);
	const int32 NumJobs = EntityCollection.GetRanges().Num();

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);

	std::atomic<int32> EntitiesProcessed = 0;
	std::atomic<int32> ConcurrentAccess = 0;
	std::atomic<int32> JobsExecuted = 0;
	FCriticalSection ParallelDetectionCS;

	FMassExecutionContext Context = EntityManager->CreateExecutionContext(/*DeltaSeconds=*/0.f);
	Context.SetEntityCollection(EntityCollection);

	Query.ParallelForEachEntityChunk(Context, [&JobsExecuted, &ParallelDetectionCS, &EntitiesProcessed, &ConcurrentAccess](FMassExecutionContext& Context)
		{
			++JobsExecuted;
			EntitiesProcessed += Context.GetNumEntities();

			if (ParallelDetectionCS.TryLock() == false)
			{
				++ConcurrentAccess;
				return;
			}
			FPlatformProcess::Sleep(0.01f);
		}, FMassEntityQuery::EParallelExecutionFlags::Force);

	CHECK(EntitiesProcessed.load() == EntitiesToProcess.Num());
	CHECK(NumJobs == JobsExecuted.load());
	CHECK(ConcurrentAccess.load() > 0);
}

#if WITH_MASSENTITY_DEBUG
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ParallelForCommands", "[Mass][Query][Parallel][Debug]")
{
	REQUIRE(EntityManager);

	const FMassArchetypeHandle TargetArchetype = EntityManager->CreateArchetype(FloatsArchetype, { FTestFragment_Tag::StaticStruct() });

	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();

	const int32 NumEntitiesToCreate = 32 * EntitiesPerChunk;
	TArray<FMassEntityHandle> CreatedEntities;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumEntitiesToCreate, CreatedEntities);

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
	Query.SetParallelCommandBufferEnabled(true);

	const int32 OriginalEntityCount = EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype);
	CHECK(OriginalEntityCount == NumEntitiesToCreate);

	FMassExecutionContext Context = EntityManager->CreateExecutionContext(/*DeltaSeconds=*/0.f);
	Query.ParallelForEachEntityChunk(Context, [](FMassExecutionContext& Context)
		{
			Context.Defer().PushCommand<FMassCommandAddTags<FTestFragment_Tag>>(Context.GetEntities());
		}, FMassEntityQuery::EParallelExecutionFlags::Force);

	const int32 OriginalArchetypeCountAfterMove = EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype);
	CHECK(OriginalArchetypeCountAfterMove == 0);
	const int32 TargetArchetypeCountAfterMove = EntityManager->DebugGetArchetypeEntitiesCount(TargetArchetype);
	CHECK(OriginalEntityCount == TargetArchetypeCountAfterMove);
}
#endif // WITH_MASSENTITY_DEBUG

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
