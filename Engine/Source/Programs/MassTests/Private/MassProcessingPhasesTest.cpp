// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

//----------------------------------------------------------------------//
// This test verifies the basic phase-ticking setup works: the processor
// gets ticked once per tick, and the count matches the tick index.
//----------------------------------------------------------------------//
TEST_CASE_METHOD(FMassLLTProcessingPhasesFixture, "Mass::ProcessingPhases::SetupTest", "[Mass][ProcessingPhases]")
{
	UMassLLTStaticCounterProcessor::StaticCounter = -1;

	PhasesConfig[0].ProcessorCDOs.Add(GetMutableDefault<UMassLLTStaticCounterProcessor>());
	InitializePhaseManager();

	for (int32 TickIdx = 0; TickIdx < 4; ++TickIdx)
	{
		Tick();
	}

	WaitForCompletion();
	INFO("Expecting the UMassLLTStaticCounterProcessor getting ticked as many times as the test ticked");
	CHECK(UMassLLTStaticCounterProcessor::StaticCounter == TickIndex);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
