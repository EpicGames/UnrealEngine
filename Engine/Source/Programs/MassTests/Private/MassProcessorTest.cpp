// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"

#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "MassExecutor.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Processor::NoEntities", "[Mass][Processor]")
{
	REQUIRE(EntityManager);

	int32 EntityProcessedCount = SimpleProcessorRun<UMassLLTProcessor_Floats>(*EntityManager);
	INFO("No entities have been created yet so the processor shouldn't do any work.");
	CHECK(EntityProcessedCount == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Processor::NoMatchingEntities", "[Mass][Processor]")
{
	REQUIRE(EntityManager);

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(IntsArchetype, 100, EntitiesCreated);
	int32 EntityProcessedCount = SimpleProcessorRun<UMassLLTProcessor_Floats>(*EntityManager);
	INFO("No matching entities have been created yet so the processor shouldn't do any work.");
	CHECK(EntityProcessedCount == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Processor::OneMatchingArchetype", "[Mass][Processor]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesToCreate = 100;
	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesToCreate, EntitiesCreated);
	int32 EntityProcessedCount = SimpleProcessorRun<UMassLLTProcessor_Floats>(*EntityManager);
	INFO("No matching entities have been created yet so the processor shouldn't do any work.");
	CHECK(EntityProcessedCount == EntitiesToCreate);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Processor::MultipleMatchingArchetype", "[Mass][Processor]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesToCreate = 100;
	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesToCreate, EntitiesCreated);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, EntitiesToCreate, EntitiesCreated);
	EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToCreate, EntitiesCreated);
	// note that only two of these archetypes match
	int32 EntityProcessedCount = SimpleProcessorRun<UMassLLTProcessor_Floats>(*EntityManager);
	INFO("Two of given three archetypes should match.");
	CHECK(EntityProcessedCount == EntitiesToCreate * 2);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
