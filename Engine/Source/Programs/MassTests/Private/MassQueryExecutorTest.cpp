// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "MassExecutor.h"
#include "MassQueryExecutor.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

//-----------------------------------------------------------------------------
// Query executor types for IteratorConsistency test
//-----------------------------------------------------------------------------

struct FTestQueryExecutor_AnyTag : public UE::Mass::FQueryExecutor
{
	int32 EntityCount = 0;

	UE::Mass::FQueryDefinition<
		UE::Mass::FConstFragmentAccess<FTestFragment_Float>
	> Accessors{ *this };

	virtual void Execute(FMassExecutionContext& Context) override
	{
		ForEachEntity(Context, Accessors, [this](FMassExecutionContext& Context, auto& Data, uint32 EntityIndex)
		{
			++EntityCount;
		});
	}
};

struct FTestQueryExecutor_NeedTagA : public UE::Mass::FQueryExecutor
{
	UE::Mass::FQueryDefinition<
		UE::Mass::FConstFragmentAccess<FTestFragment_Float>,
		UE::Mass::FMassTagRequired<FTestTag_A>
	> Accessors{ *this };

	int32 EntityCount = 0;

	virtual void Execute(FMassExecutionContext& Context) override
	{
		ForEachEntity(Context, Accessors, [this](FMassExecutionContext& Context, auto& Data, uint32 EntityIndex)
		{
			++EntityCount;
		});
	}
};

struct FTestQueryExecutor_BlockTagB : public UE::Mass::FQueryExecutor
{
	UE::Mass::FQueryDefinition<
		UE::Mass::FConstFragmentAccess<FTestFragment_Float>,
		UE::Mass::FMassTagBlocked<FTestTag_B>
	> Accessors{ *this };

	int32 EntityCount = 0;

	virtual void Execute(FMassExecutionContext& Context) override
	{
		ForEachEntity(Context, Accessors, [this](FMassExecutionContext& Context, auto& Data, uint32 EntityIndex)
		{
			++EntityCount;
		});
	}
};

//-----------------------------------------------------------------------------
// Tests
//-----------------------------------------------------------------------------

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Processor::AutoExecuteQuery::IteratorConsistency", "[Mass][Processor][QueryExecutor]")
{
	REQUIRE(EntityManager);

	const int32 EntityCountNoTag = 7;
	const int32 EntityCountA = 9;
	const int32 EntityCountB = 13;
	const int32 EntityCountAB = 17;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, {}, EntityCountNoTag, Entities);

	Entities.Reset();
	EntityManager->BatchCreateEntities(FloatsArchetype, {}, EntityCountA, Entities);
	for (FMassEntityHandle& Entity : Entities)
	{
		EntityManager->AddTagToEntity(Entity, FTestTag_A::StaticStruct());
	}

	Entities.Reset();
	EntityManager->BatchCreateEntities(FloatsArchetype, {}, EntityCountB, Entities);
	for (FMassEntityHandle& Entity : Entities)
	{
		EntityManager->AddTagToEntity(Entity, FTestTag_B::StaticStruct());
	}

	Entities.Reset();
	EntityManager->BatchCreateEntities(FloatsArchetype, {}, EntityCountAB, Entities);
	for (FMassEntityHandle& Entity : Entities)
	{
		EntityManager->AddTagToEntity(Entity, FTestTag_A::StaticStruct());
		EntityManager->AddTagToEntity(Entity, FTestTag_B::StaticStruct());
	}

	TObjectPtr<UMassLLTProcessorAutoExecuteQuery> Processor_AnyTag = NewObject<UMassLLTProcessorAutoExecuteQuery>();
	REQUIRE(Processor_AnyTag);
	TSharedPtr<FTestQueryExecutor_AnyTag> TestQuery_AnyTag = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_AnyTag>(Processor_AnyTag->EntityQuery, Processor_AnyTag);
	Processor_AnyTag->SetAutoExecuteQuery(TestQuery_AnyTag);
	Processor_AnyTag->CallInitialize(GetTransientPackageAsObject(), EntityManager.ToSharedRef());

	TObjectPtr<UMassLLTProcessorAutoExecuteQuery> Processor_TagA = NewObject<UMassLLTProcessorAutoExecuteQuery>();
	REQUIRE(Processor_TagA);
	TSharedPtr<FTestQueryExecutor_NeedTagA> TestQuery_TagA = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_NeedTagA>(Processor_TagA->EntityQuery, Processor_TagA);
	Processor_TagA->SetAutoExecuteQuery(TestQuery_TagA);
	Processor_TagA->CallInitialize(GetTransientPackageAsObject(), EntityManager.ToSharedRef());

	TObjectPtr<UMassLLTProcessorAutoExecuteQuery> Processor_TagB = NewObject<UMassLLTProcessorAutoExecuteQuery>();
	REQUIRE(Processor_TagB);
	TSharedPtr<FTestQueryExecutor_BlockTagB> TestQuery_TagB = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_BlockTagB>(Processor_TagB->EntityQuery, Processor_TagB);
	Processor_TagB->SetAutoExecuteQuery(TestQuery_TagB);
	Processor_TagB->CallInitialize(GetTransientPackageAsObject(), EntityManager.ToSharedRef());

	TArray<TObjectPtr<UMassProcessor>> Processors;
	Processors.Add(Processor_AnyTag);
	Processors.Add(Processor_TagA);
	Processors.Add(Processor_TagB);

	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);

	UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);

	const int32 AnyCount = TestQuery_AnyTag->EntityCount;
	const int32 ExpectedAnyCount = EntityCountNoTag + EntityCountA + EntityCountB + EntityCountAB;
	const int32 ACount = TestQuery_TagA->EntityCount;
	const int32 ExpectedACount = EntityCountA + EntityCountAB;
	const int32 BCount = TestQuery_TagB->EntityCount;
	const int32 ExpectedBCount = EntityCountNoTag + EntityCountA;

	INFO("Any Tag Entities Processed");
	CHECK(AnyCount == ExpectedAnyCount);
	INFO("Require TagA Entities Processed");
	CHECK(ACount == ExpectedACount);
	INFO("Blocked TagB Entities Processed");
	CHECK(BCount == ExpectedBCount);
}

// NOTE: FQueryExecutor_LoadTest ("System.Mass.Processor.AutoExecuteQuery.LoadTest") is a performance
// benchmark that requires UMassTestWorldSubsystem (UWorld), and is not portable to standalone LLT.
// It remains in the Automation Framework (MassEntityTestSuite) only.

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
