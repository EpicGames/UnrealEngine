// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityManager.h"
#include "MassProcessingContext.h"
#include "MassExecutor.h"
#include "MassExecutionContext.h"
#include "MassTypeManager.h"
#include "MassEntityLinkFragments.h"
#include "Algo/Compare.h"

#include "TestHarness.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ProcessorRequirements", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	UMassLLTProcessor_Floats* Processor = NewTestProcessor<UMassLLTProcessor_Floats>(EntityManager);
	TConstArrayView<FMassFragmentRequirementDescription> Requirements = Processor->EntityQuery.GetFragmentRequirements();

	INFO("Query should have extracted some requirements from the given Processor");
	CHECK(Requirements.Num() > 0);
	INFO("There should be exactly one requirement");
	REQUIRE(Requirements.Num() == 1);
	INFO("The requirement should be of the Float fragment type");
	CHECK(Requirements[0].StructType == FTestFragment_Float::StaticStruct());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ExplicitRequirements", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Float::StaticStruct()});
	TConstArrayView<FMassFragmentRequirementDescription> Requirements = Query.GetFragmentRequirements();

	INFO("Query should have extracted some requirements from the given Processor");
	CHECK(Requirements.Num() > 0);
	INFO("There should be exactly one requirement");
	REQUIRE(Requirements.Num() == 1);
	INFO("The requirement should be of the Float fragment type");
	CHECK(Requirements[0].StructType == FTestFragment_Float::StaticStruct());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::FragmentViewBinding", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
	FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
	INFO("Initial value of the fragment should match expectations");
	CHECK(TestedFragment.Value == 0.f);

	UMassLLTProcessor_Floats* Processor = NewTestProcessor<UMassLLTProcessor_Floats>(EntityManager);
	Processor->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context)
	{
		TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();
		for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
		{
			Floats[EntityIndex].Value = 13.f;
		}
	};

	FMassProcessingContext ProcessingContext(*EntityManager, /*DeltaSeconds=*/0.f);
	UE::Mass::Executor::Run(*Processor, ProcessingContext);

	INFO("Fragment value should have changed to the expected value");
	CHECK(TestedFragment.Value == 13.f);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ExecuteSingleArchetype", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	const int32 NumToCreate = 10;
	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumToCreate, EntitiesCreated);

	int32 TotalProcessed = 0;

	FMassExecutionContext ExecContext(*EntityManager.Get());
	FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Float::StaticStruct() });
	Query.ForEachEntityChunk(ExecContext, [&TotalProcessed](FMassExecutionContext& Context)
		{
			TotalProcessed += Context.GetNumEntities();
			TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				Floats[EntityIndex].Value = 13.f;
			}
		});

	INFO("The number of entities processed needs to match expectations");
	CHECK(TotalProcessed == NumToCreate);

	for (FMassEntityHandle& Entity : EntitiesCreated)
	{
		const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		INFO("Every fragment value should have changed to the expected value");
		CHECK(TestedFragment.Value == 13.f);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ExecuteMultipleArchetypes", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	const int32 FloatsArchetypeCreated = 7;
	const int32 IntsArchetypeCreated = 11;
	const int32 FloatsIntsArchetypeCreated = 13;
	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(IntsArchetype, IntsArchetypeCreated, EntitiesCreated);
	// clear to store only the float-related entities
	EntitiesCreated.Reset();
	EntityManager->BatchCreateEntities(FloatsArchetype, FloatsArchetypeCreated, EntitiesCreated);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, FloatsIntsArchetypeCreated, EntitiesCreated);

	int32 TotalProcessed = 0;
	FMassExecutionContext ExecContext(*EntityManager.Get());
	FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Float::StaticStruct() });
	Query.ForEachEntityChunk(ExecContext, [&TotalProcessed](FMassExecutionContext& Context)
		{
			TotalProcessed += Context.GetNumEntities();
			TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				Floats[EntityIndex].Value = 13.f;
			}
		});

	INFO("The number of entities processed needs to match expectations");
	CHECK(TotalProcessed == FloatsIntsArchetypeCreated + FloatsArchetypeCreated);

	for (FMassEntityHandle& Entity : EntitiesCreated)
	{
		const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		INFO("Every fragment value should have changed to the expected value");
		CHECK(TestedFragment.Value == 13.f);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ExecuteSparse", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	const int32 NumToCreate = 10;
	TArray<FMassEntityHandle> AllEntitiesCreated;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumToCreate, AllEntitiesCreated);

	TArray<int32> IndicesToProcess = { 1, 2, 3, 6, 7};
	TArray<FMassEntityHandle> EntitiesToProcess;
	TArray<FMassEntityHandle> EntitiesToIgnore;
	for (int32 EntityIndex = 0; EntityIndex < AllEntitiesCreated.Num(); ++EntityIndex)
	{
		if (IndicesToProcess.Find(EntityIndex) != INDEX_NONE)
		{
			EntitiesToProcess.Add(AllEntitiesCreated[EntityIndex]);
		}
		else
		{
			EntitiesToIgnore.Add(AllEntitiesCreated[EntityIndex]);
		}
	}

	int32 TotalProcessed = 0;

	FMassExecutionContext ExecContext(*EntityManager.Get());
	FMassEntityQuery TestQuery(EntityManager);
	TestQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
	TestQuery.ForEachEntityChunk(FMassArchetypeEntityCollection(FloatsArchetype, EntitiesToProcess, FMassArchetypeEntityCollection::NoDuplicates)
							, ExecContext, [&TotalProcessed](FMassExecutionContext& Context)
		{
			TotalProcessed += Context.GetNumEntities();
			TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				Floats[EntityIndex].Value = 13.f;
			}
		});

	INFO("The number of entities processed needs to match expectations");
	CHECK(TotalProcessed == IndicesToProcess.Num());

	for (FMassEntityHandle& Entity : EntitiesToProcess)
	{
		const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		INFO("Every fragment value should have changed to the expected value");
		CHECK(TestedFragment.Value == 13.f);
	}

	for (FMassEntityHandle& Entity : EntitiesToIgnore)
	{
		const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		INFO("Untouched entities should retain default fragment value");
		CHECK(TestedFragment.Value == 0.f);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::TagPresent", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	TArray<const UScriptStruct*> Fragments = {FTestFragment_Float::StaticStruct(), FTestFragment_Tag::StaticStruct()};
	const FMassArchetypeHandle FloatsTagArchetype = EntityManager->CreateArchetype(Fragments);

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
	Query.AddTagRequirement<FTestFragment_Tag>(EMassFragmentPresence::All);
	Query.CacheArchetypes();

	INFO("There's a single archetype matching the requirements");
	REQUIRE(Query.GetArchetypes().Num() == 1);
	INFO("The only valid archetype is FloatsTagArchetype");
	CHECK(FloatsTagArchetype == Query.GetArchetypes()[0]);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::TagAbsent", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	TArray<const UScriptStruct*> Fragments = { FTestFragment_Float::StaticStruct(), FTestFragment_Tag::StaticStruct() };
	const FMassArchetypeHandle FloatsTagArchetype = EntityManager->CreateArchetype(Fragments);

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
	Query.AddTagRequirement<FTestFragment_Tag>(EMassFragmentPresence::None);
	Query.CacheArchetypes();

	INFO("There are exactly two archetypes matching the requirements");
	REQUIRE(Query.GetArchetypes().Num() == 2);
	INFO("FloatsTagArchetype is not amongst matching archetypes");
	CHECK(!(FloatsTagArchetype == Query.GetArchetypes()[0] || FloatsTagArchetype == Query.GetArchetypes()[1]));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::FragmentPresent", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityQuery Query(EntityManager);
	// using EMassFragmentAccess::None to indicate we're interested only in the archetype having the fragment, no binding is required
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::Any);
	Query.CacheArchetypes();

	INFO("There are exactly two archetypes matching the requirements");
	REQUIRE(Query.GetArchetypes().Num() == 2);
	INFO("FloatsArchetype is not amongst matching archetypes");
	CHECK(!(FloatsArchetype == Query.GetArchetypes()[0] || FloatsArchetype == Query.GetArchetypes()[1]));

	constexpr int32 NumberOfEntitiesToAddA = 5;
	constexpr int32 NumberOfEntitiesToAddB = 7;
	TArray<FMassEntityHandle> MatchingEntities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumberOfEntitiesToAddA, MatchingEntities);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumberOfEntitiesToAddB, MatchingEntities);
	CHECK(MatchingEntities.Num() == NumberOfEntitiesToAddA + NumberOfEntitiesToAddB);

	int32 TotalProcessed = 0;
	FMassExecutionContext ExecContext(*EntityManager.Get());
	Query.ForEachEntityChunk(ExecContext, [&TotalProcessed](FMassExecutionContext& Context) {
		TotalProcessed += Context.GetNumEntities();
	});
	INFO("We expect the number of entities processed to match number added to matching archetypes");
	CHECK(MatchingEntities.Num() == TotalProcessed);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::OnlyAbsentFragments", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityQuery Query(EntityManager);
	INFO("The empty query is not valid");
	CHECK_FALSE(Query.CheckValidity());

	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
	INFO("Single negative requirement is valid");
	CHECK(Query.CheckValidity());

	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
	Query.AddRequirement<FTestFragment_Large>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
	INFO("Multiple negative requirement is valid");
	CHECK(Query.CheckValidity());

	Query.CacheArchetypes();
	INFO("There's only one default test archetype matching the query");
	REQUIRE(Query.GetArchetypes().Num() == 1);
	INFO("Only the Empty archetype matches the query");
	CHECK(Query.GetArchetypes()[0] == EmptyArchetype);

	const FMassArchetypeHandle NewMatchingArchetypeHandle = EntityManager->CreateArchetype({ FTestFragment_Bool::StaticStruct() });
	Query.CacheArchetypes();
	INFO("The number of matching queries matches expectations");
	REQUIRE(Query.GetArchetypes().Num() == 2);
	INFO("The new archetype matches the query");
	CHECK(Query.GetArchetypes()[1] == NewMatchingArchetypeHandle);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::AbsentAndPresentFragments", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);

	INFO("The query is valid");
	CHECK(Query.CheckValidity());
	Query.CacheArchetypes();
	INFO("There is only one archetype matching the query");
	REQUIRE(Query.GetArchetypes().Num() == 1);
	INFO("FloatsArchetype is the only one matching the query");
	CHECK(FloatsArchetype == Query.GetArchetypes()[0]);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::SingleOptionalFragment", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	Query.CacheArchetypes();

	INFO("There are exactly two archetypes matching the requirements");
	REQUIRE(Query.GetArchetypes().Num() == 2);
	INFO("FloatsArchetype is not amongst matching archetypes");
	CHECK(!(FloatsArchetype == Query.GetArchetypes()[0] || FloatsArchetype == Query.GetArchetypes()[1]));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::MultipleOptionalFragment", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	Query.CacheArchetypes();

	INFO("All three archetype meet requirements");
	CHECK(Query.GetArchetypes().Num() == 3);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::UsingOptionalFragment", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	EntityManager->CreateEntity(FloatsArchetype);
	const FMassEntityHandle EntityWithFloatsInts = EntityManager->CreateEntity(FloatsIntsArchetype);
	EntityManager->CreateEntity(IntsArchetype);

	const int32 IntValueSet = 123;
	int32 TotalProcessed = 0;
	int32 EmptyIntsViewCount = 0;

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
	FMassExecutionContext ExecContext(*EntityManager.Get());
	Query.ForEachEntityChunk(ExecContext, [&TotalProcessed, &EmptyIntsViewCount, IntValueSet](FMassExecutionContext& Context) {
		++TotalProcessed;
		TArrayView<FTestFragment_Int> Ints = Context.GetMutableFragmentView<FTestFragment_Int>();
		if (Ints.Num() == 0)
		{
			++EmptyIntsViewCount;
		}
		else
		{
			for (FTestFragment_Int& IntFragment : Ints)
			{
				IntFragment.Value = IntValueSet;
			}
		}
		});

	INFO("Two archetypes total should get processed");
	CHECK(TotalProcessed == 2);
	INFO("Only one of these archetypes should get an empty Ints array view");
	CHECK(EmptyIntsViewCount == 1);

	const FTestFragment_Int& TestFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(EntityWithFloatsInts);
	INFO("The optional fragment's value should get modified where present");
	CHECK(TestFragment.Value == IntValueSet);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::AnyFragment", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	// From FMassLLTEntityFixture:
	// FMassArchetypeHandle FloatsArchetype;
	// FMassArchetypeHandle IntsArchetype;
	// FMassArchetypeHandle FloatsIntsArchetype;
	const FMassArchetypeHandle BoolArchetype = EntityManager->CreateArchetype({ FTestFragment_Bool::StaticStruct() });
	const FMassArchetypeHandle BoolFloatArchetype = EntityManager->CreateArchetype({ FTestFragment_Bool::StaticStruct(), FTestFragment_Float::StaticStruct() });

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Any);
	Query.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Any);
	// this query should match:
	// IntsArchetype, FloatsIntsArchetype, BoolArchetype, BoolFloatArchetype
	Query.CacheArchetypes();

	INFO("Archetypes containing Int or Bool should meet requirements");
	CHECK(Query.GetArchetypes().Num() == 4);

	// populate the archetypes so that we can test fragment binding
	for (auto ArchetypeHandle : Query.GetArchetypes())
	{
		EntityManager->CreateEntity(ArchetypeHandle);
	}

	FMassExecutionContext TestContext(*EntityManager.Get());
	Query.ForEachEntityChunk(TestContext, [](FMassExecutionContext& Context)
		{
			TArrayView<FTestFragment_Bool> BoolView = Context.GetMutableFragmentView<FTestFragment_Bool>();
			TArrayView<FTestFragment_Int> IntView = Context.GetMutableFragmentView<FTestFragment_Int>();

			INFO("Every matching archetype needs to host Bool or Int fragments");
			CHECK((BoolView.Num() || IntView.Num()));
		});
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::AnyTag", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	const FMassArchetypeHandle ABArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct(), FTestTag_B::StaticStruct() });
	const FMassArchetypeHandle ACArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct(), FTestTag_C::StaticStruct() });
	const FMassArchetypeHandle BCArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_B::StaticStruct(), FTestTag_C::StaticStruct() });
	const FMassArchetypeHandle BDArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_B::StaticStruct(), FTestTag_D::StaticStruct() });
	const FMassArchetypeHandle FloatACArchetype = EntityManager->CreateArchetype({ FTestFragment_Float::StaticStruct(), FTestTag_A::StaticStruct(), FTestTag_C::StaticStruct() });

	FMassEntityQuery Query(EntityManager);
	// at least one fragment requirement needs to be present for the query to be valid
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	Query.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::Any);
	Query.AddTagRequirement<FTestTag_C>(EMassFragmentPresence::Any);
	// this query should match:
	// ABArchetype, ACArchetype and BCArchetype but not BDArchetype nor IntsArchetype
	Query.CacheArchetypes();

	INFO("Only Archetypes tagged with A or C should matched the query");
	CHECK(Query.GetArchetypes().Num() == 3);
	INFO("ABArchetype should be amongst the matched archetypes");
	CHECK(Query.GetArchetypes().Find(ABArchetype) != INDEX_NONE);
	INFO("ACArchetype should be amongst the matched archetypes");
	CHECK(Query.GetArchetypes().Find(ACArchetype) != INDEX_NONE);
	INFO("BCArchetype should be amongst the matched archetypes");
	CHECK(Query.GetArchetypes().Find(BCArchetype) != INDEX_NONE);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::AutoRecache", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityQuery Query(EntityManager);
	// at least one fragment requirement needs to be present for the query to be valid
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);

	int32 EntitiesFound = 0;
	const FMassExecuteFunction QueryExecFunction = [&EntitiesFound](FMassExecutionContext& Context)
	{
		EntitiesFound += Context.GetNumEntities();
	};

	FMassExecutionContext ExecutionContext(*EntityManager, /*DeltaSeconds=*/0.f);
	Query.ForEachEntityChunk(ExecutionContext, QueryExecFunction);

	INFO("No entities have been created so we expect counting to yield 0");
	CHECK(EntitiesFound == 0);

	constexpr int32 NumberOfEntitiesMatching = 17;
	TArray<FMassEntityHandle> MatchingEntities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumberOfEntitiesMatching, MatchingEntities);

	EntitiesFound = 0;
	Query.ForEachEntityChunk(ExecutionContext, QueryExecFunction);
	INFO("The number of entities found should match the number of entities created in the matching archetype");
	CHECK(EntitiesFound == MatchingEntities.Num());

	// create more entities, but in an archetype not matching the query
	constexpr int32 NumberOfEntitiesNotMatching = 13;
	TArray<FMassEntityHandle> NotMatchingEntities;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumberOfEntitiesNotMatching, NotMatchingEntities);
	EntitiesFound = 0;
	Query.ForEachEntityChunk(ExecutionContext, QueryExecFunction);
	INFO("The number of entities found should not change with addition of entities not matching the query");
	CHECK(EntitiesFound == MatchingEntities.Num());

	// create some more in another matching archetype
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumberOfEntitiesMatching, MatchingEntities);
	EntitiesFound = 0;
	Query.ForEachEntityChunk(ExecutionContext, QueryExecFunction);
	INFO("The total number of entities found should include entities from both matching archetypes");
	CHECK(EntitiesFound == MatchingEntities.Num());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::AllOptional", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::None, EMassFragmentPresence::Optional);
	Query.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::Optional);
	Query.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::Optional);
	Query.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::Optional);
	Query.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::Optional);

	Query.CacheArchetypes();

	int32 ExpectedNumOfArchetypes = 2;
	// only the FloatsArchetype and FloatsIntsArchetype should match
	INFO("Initial number of matching archetypes matches expectations");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);

	int32 CurrentEntityIndex = 0;

	EntityManager->AddTagToEntity(Entities[CurrentEntityIndex++], FTestTag_A::StaticStruct());
	++ExpectedNumOfArchetypes;
	EntityManager->AddTagToEntity(Entities[CurrentEntityIndex++], FTestTag_B::StaticStruct());
	Query.CacheArchetypes();
	INFO("A: number of matching archetypes matches expectations.");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	{
		FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
		Descriptor.Add<FTestChunkFragment_Int>();
		EntityManager->CreateArchetype(Descriptor);
		++ExpectedNumOfArchetypes;
	}
	{
		FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
		Descriptor.Add<FTestChunkFragment_Float>();
		EntityManager->CreateArchetype(Descriptor);
	}
	Query.CacheArchetypes();
	INFO("B: number of matching archetypes matches expectations.");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	{
		FTestSharedFragment_Int FragmentInstance;
		FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
		FMassArchetypeSharedFragmentValues SharedFragmentValues;
		SharedFragmentValues.Add(SharedFragmentInstance);

		FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&Entities[CurrentEntityIndex++], 1), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
		EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&Collection, 1), SharedFragmentValues);
		++ExpectedNumOfArchetypes;
	}
	{
		FTestSharedFragment_Float FragmentInstance;
		FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
		FMassArchetypeSharedFragmentValues SharedFragmentValues;
		SharedFragmentValues.Add(SharedFragmentInstance);

		FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&Entities[CurrentEntityIndex++], 1), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
		EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&Collection, 1), SharedFragmentValues);
	}
	Query.CacheArchetypes();
	INFO("C: number of matching archetypes matches expectations.");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	{
		FTestConstSharedFragment_Int FragmentInstance;
		FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
		EntityManager->AddConstSharedFragmentToEntity(Entities[CurrentEntityIndex++], SharedFragmentInstance);
		++ExpectedNumOfArchetypes;
	}
	{
		FTestConstSharedFragment_Float FragmentInstance;
		FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
		EntityManager->AddConstSharedFragmentToEntity(Entities[CurrentEntityIndex++], SharedFragmentInstance);
	}
	Query.CacheArchetypes();
	INFO("D: number of matching archetypes matches expectations.");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::JustATag", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassEntityQuery Query(EntityManager);
	Query.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
	Query.CacheArchetypes();

	int32 ExpectedNumOfArchetypes = 0;
	INFO("Initial number of matching archetypes matches expectations");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	{
		FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
		Descriptor.Add<FTestTag_A>();
		EntityManager->CreateArchetype(Descriptor);
		++ExpectedNumOfArchetypes;
	}
	{
		FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
		Descriptor.Add<FTestTag_B>();
		EntityManager->CreateArchetype(Descriptor);
	}
	Query.CacheArchetypes();
	INFO("A: number of matching archetypes matches expectations.");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	{
		FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
		Descriptor.Add<FTestTag_A>();
		Descriptor.Add<FTestTag_C>();
		Descriptor.Add<FTestTag_D>();
		EntityManager->CreateArchetype(Descriptor);
		++ExpectedNumOfArchetypes;
	}
	{
		FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
		Descriptor.Add<FTestTag_B>();
		Descriptor.Add<FTestTag_C>();
		Descriptor.Add<FTestTag_D>();
		EntityManager->CreateArchetype(Descriptor);
	}
	Query.CacheArchetypes();
	INFO("B: number of matching archetypes matches expectations.");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::JustAChunkFragment", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassArchetypeHandle TargetArchetype;

	FMassEntityQuery Query(EntityManager);
	Query.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Query.CacheArchetypes();

	int32 ExpectedNumOfArchetypes = 0;
	// no matching archetypes at this time
	INFO("Initial number of matching archetypes matches expectations");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	{
		FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
		Descriptor.Add<FTestChunkFragment_Int>();
		TargetArchetype = EntityManager->CreateArchetype(Descriptor);
		++ExpectedNumOfArchetypes;
	}
	{
		FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
		Descriptor.Add<FTestChunkFragment_Float>();
		EntityManager->CreateArchetype(Descriptor);
	}
	Query.CacheArchetypes();
	INFO("Number of matching archetypes matches expectations.");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	// try to access the chunk fragment
	{
		EntityManager->CreateEntity(TargetArchetype);

		FMassExecutionContext ExecContext(*EntityManager.Get());
		bool bExecuted = false;
		Query.ForEachEntityChunk(ExecContext, [&bExecuted](FMassExecutionContext& Context)
			{
				const FTestChunkFragment_Int& ChunkFragment = Context.GetChunkFragment<FTestChunkFragment_Int>();
				bExecuted = true;
			});
		INFO("The tested query did execute and bounding was successful");
		CHECK(bExecuted);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::JustASharedFragment", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassArchetypeHandle TargetArchetype;

	FMassEntityQuery Query(EntityManager);
	Query.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Query.CacheArchetypes();

	int32 ExpectedNumOfArchetypes = 0;
	// no matching archetypes at this time
	INFO("Initial number of matching archetypes matches expectations");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);
	int32 CurrentEntityIndex = 0;

	{
		FTestSharedFragment_Int FragmentInstance;
		FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
		FMassArchetypeSharedFragmentValues SharedFragmentValues;
		SharedFragmentValues.Add(SharedFragmentInstance);

		FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&Entities[CurrentEntityIndex++], 1), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
		EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&Collection, 1), SharedFragmentValues);
		++ExpectedNumOfArchetypes;
	}
	{
		FTestSharedFragment_Float FragmentInstance;
		FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
		FMassArchetypeSharedFragmentValues SharedFragmentValues;
		SharedFragmentValues.Add(SharedFragmentInstance);

		FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&Entities[CurrentEntityIndex++], 1), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
		EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&Collection, 1), SharedFragmentValues);
	}
	Query.CacheArchetypes();
	INFO("Number of matching archetypes matches expectations.");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	// try to access the shared fragment
	{
		bool bExecuted = false;
		FMassExecutionContext ExecContext(*EntityManager.Get());
		Query.ForEachEntityChunk(ExecContext, [&bExecuted](FMassExecutionContext& Context)
			{
				const FTestSharedFragment_Int& SharedFragment = Context.GetSharedFragment<FTestSharedFragment_Int>();
				bExecuted = true;
			});
		INFO("The tested query did execute and bounding was successful");
		CHECK(bExecuted);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::JustAConstSharedFragment", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	FMassArchetypeHandle TargetArchetype;

	FMassEntityQuery Query(EntityManager);
	Query.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::All);
	Query.CacheArchetypes();

	int32 ExpectedNumOfArchetypes = 0;
	// no matching archetypes at this time
	INFO("Initial number of matching archetypes matches expectations");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);
	int32 CurrentEntityIndex = 0;

	{
		FTestConstSharedFragment_Int FragmentInstance;
		FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
		EntityManager->AddConstSharedFragmentToEntity(Entities[CurrentEntityIndex++], SharedFragmentInstance);
		++ExpectedNumOfArchetypes;
	}
	{
		FTestConstSharedFragment_Float FragmentInstance;
		FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
		EntityManager->AddConstSharedFragmentToEntity(Entities[CurrentEntityIndex++], SharedFragmentInstance);
	}
	Query.CacheArchetypes();
	INFO("Number of matching archetypes matches expectations.");
	CHECK(Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

	// try to access the shared fragment
	{
		bool bExecuted = false;
		FMassExecutionContext ExecContext(*EntityManager.Get());
		Query.ForEachEntityChunk(ExecContext, [&bExecuted](FMassExecutionContext& Context)
			{
				const FTestConstSharedFragment_Int& SharedFragment = Context.GetConstSharedFragment<FTestConstSharedFragment_Int>();
				bExecuted = true;
			});
		INFO("The tested query did execute and bounding was successful");
		CHECK(bExecuted);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::GameThreadOnly", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	EntityManager->GetTypeManager().RegisterType<FTestSharedFragment_Int>();
	EntityManager->GetTypeManager().RegisterType<UMassLLTWorldSubsystem>();

	{
		FMassEntityQuery Query(EntityManager);
		Query.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
		INFO("Statically typed shared fragment");
		CHECK(Query.DoesRequireGameThreadExecution() == TMassSharedFragmentTraits<FTestSharedFragment_Int>::GameThreadOnly);
	}
	{
		FMassEntityQuery Query(EntityManager);
		Query.AddSharedRequirement(FTestSharedFragment_Int::StaticStruct(), EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
		INFO("Statically typed shared fragment");
		CHECK(Query.DoesRequireGameThreadExecution() == TMassSharedFragmentTraits<FTestSharedFragment_Int>::GameThreadOnly);
	}
	{
		FMassEntityQuery Query(EntityManager);
		Query.AddSubsystemRequirement<UMassLLTWorldSubsystem>(EMassFragmentAccess::ReadWrite);
		INFO("Statically typed shared fragment");
		CHECK(Query.DoesRequireGameThreadExecution() == TMassSharedFragmentTraits<UMassLLTWorldSubsystem>::GameThreadOnly);
	}
	{
		FMassEntityQuery Query(EntityManager);
		Query.AddSubsystemRequirement(UMassLLTWorldSubsystem::StaticClass(), EMassFragmentAccess::ReadWrite);
		INFO("Statically typed shared fragment");
		CHECK(Query.DoesRequireGameThreadExecution() == TMassSharedFragmentTraits<UMassLLTWorldSubsystem>::GameThreadOnly);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ExportHandles", "[Mass][Query]")
{
	constexpr int32 EntitiesPerChunk = 16384;
	constexpr int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);
	CHECK(Entities.Num() == 2*Count);

	TArray<FMassArchetypeEntityCollection> EntityCollections;
	UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
	EntityManager->BatchChangeTagsForEntities(EntityCollections, FMassTagBitSet(FTestTag_A::StaticStruct()), FMassTagBitSet());

	FMassEntityQuery Query(EntityManager.ToSharedRef());
	Query.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);

	TArray<FMassEntityHandle> QueryMatchingEntities = Query.GetMatchingEntityHandles();

	Entities.Sort();
	QueryMatchingEntities.Sort();
	INFO("Exported handle list contain all the expected handles");
	CHECK(Algo::Compare(Entities, QueryMatchingEntities));

	TArray<FMassArchetypeEntityCollection> MatchingCollections = Query.CreateMatchingEntitiesCollection();
	INFO("Expected number of archetypes in resulting collections");
	CHECK(MatchingCollections.Num() == 2);

	TArray<FMassEntityHandle> HandlesFromCollections;
	for (const FMassArchetypeEntityCollection& Collection : MatchingCollections)
	{
		Collection.ExportEntityHandles(HandlesFromCollections);
	}
	HandlesFromCollections.Sort();
	INFO("Handles exported from the collections contain all the expected handles");
	CHECK(Algo::Compare(Entities, HandlesFromCollections));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::ExecutionLimiter", "[Mass][Query]")
{
	REQUIRE(EntityManager);

	const int32 EntityLimit = 1;
	const int32 CyclesToSaturate = 20; // no assumptions about chunk size, the minimum cycles will be the total entity count needing processing
	const int32 FloatsArchetypeCreated = 7;
	const int32 IntsArchetypeCreated = 11;
	const int32 FloatsIntsArchetypeCreated = 13;
	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(IntsArchetype, IntsArchetypeCreated, EntitiesCreated);
	// clear to store only the float-related entities
	EntitiesCreated.Reset();
	EntityManager->BatchCreateEntities(FloatsArchetype, FloatsArchetypeCreated, EntitiesCreated);
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, FloatsIntsArchetypeCreated, EntitiesCreated);

	int32 TotalProcessed = 0;
	FMassExecutionContext ExecContext(*EntityManager.Get());
	FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Float::StaticStruct() });
	// limit the execution to a single entity per cycle (this will process the entire chunk, but only one no matter how many entities it contains)
	UE::Mass::FExecutionLimiter Limiter(EntityLimit);
	Query.ForEachEntityChunk(ExecContext, Limiter, [&TotalProcessed](FMassExecutionContext& Context)
		{
			TotalProcessed += Context.GetNumEntities();
			TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				Floats[EntityIndex].Value = 13.f;
			}
		});

	INFO("Limiter should prevent all entities from being processed");
	CHECK(TotalProcessed < FloatsIntsArchetypeCreated + FloatsArchetypeCreated);
	INFO("Limiter should process at least as many entities as requested");
	CHECK(TotalProcessed >= EntityLimit);

	// run a few iterations to ensure all entities are processed:
	for (int32 CycleIndex = 0; CycleIndex < CyclesToSaturate; CycleIndex++)
	{
		Query.ForEachEntityChunk(ExecContext, Limiter, [&TotalProcessed](FMassExecutionContext& Context)
			{
				TotalProcessed += Context.GetNumEntities();
				TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

				for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
				{
					Floats[EntityIndex].Value = 13.f;
				}
			});
	}

	for (FMassEntityHandle& Entity : EntitiesCreated)
	{
		const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		INFO("Every fragment value should have changed to the expected value");
		CHECK(TestedFragment.Value == 13.f);
	}

	Limiter.EntityLimit = 1000000;
	TotalProcessed = 0;
	Query.ForEachEntityChunk(ExecContext, Limiter, [&TotalProcessed](FMassExecutionContext& Context)
		{
			TotalProcessed += Context.GetNumEntities();
			TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				Floats[EntityIndex].Value = 13.f;
			}
		});

	INFO("All entities should be processed with a high enough limit");
	CHECK(TotalProcessed == FloatsIntsArchetypeCreated + FloatsArchetypeCreated);

	for (FMassEntityHandle& Entity : EntitiesCreated)
	{
		const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		INFO("Every fragment value should have changed to the expected value");
		CHECK(TestedFragment.Value == 13.f);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::LinkedFragmentAccess", "[Mass][Query]")
{
	REQUIRE(EntityManager);
	const int32 DirectIntValue = 1234;
	const int32 LinkedIntValue = 9898;
	const float DirectFloatValue = 10.0f;
	const float InirectFloatValue = -10.0f;

	TArray<const UScriptStruct*> Fragments = { FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct() };
	const FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(Fragments);
	FMassEntityHandle LinkedEntity = EntityManager->CreateEntity(Archetype);
	FTestFragment_Float& LinkedFloatFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(LinkedEntity);
	FTestFragment_Int& LinkedIntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(LinkedEntity);
	LinkedFloatFragment.Value = InirectFloatValue;
	LinkedIntFragment.Value = LinkedIntValue;

	FMassEntityLinkFragment LinkedEntityFragment;
	LinkedEntityFragment.LinkedEntityHandle = LinkedEntity;
	FConstSharedStruct SharedLinkedEntityFragment = EntityManager->GetOrCreateConstSharedFragment(LinkedEntityFragment);

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);
	for (FMassEntityHandle& Entity : Entities)
	{
		EntityManager->AddConstSharedFragmentToEntity(Entity, SharedLinkedEntityFragment);
		FTestFragment_Int& DirectIntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity);
		DirectIntFragment.Value = DirectIntValue;
	}

	Entities.Reset();
	EntityManager->BatchCreateEntities(FloatsArchetype, 10, Entities);
	for (FMassEntityHandle& Entity : Entities)
	{
		EntityManager->AddConstSharedFragmentToEntity(Entity, SharedLinkedEntityFragment);
		FTestFragment_Float& DirectFloatFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		DirectFloatFragment.Value = DirectFloatValue;
	}

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	Query.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadOnly);
	Query.CacheArchetypes();

	bool bAnyMatch = false;
	bool bFoundShared = true;
	FMassExecutionContext ExecContext(*EntityManager.Get());
	Query.ForEachEntityChunk(ExecContext, [&bAnyMatch, &bFoundShared](FMassExecutionContext& Context)
		{
			const FTestFragment_Int* SharedInt = Context.GetLinkedEntityConstFragmentPtr<FTestFragment_Int>();
			if (!SharedInt)
			{
				bFoundShared = false;
				return;
			}
			const TConstArrayView<FTestFragment_Int> IntFragments = Context.GetFragmentView<FTestFragment_Int>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FTestFragment_Int& IntFragment = IntFragments[EntityIt];
				bAnyMatch |= IntFragment.Value == SharedInt->Value;
			}
		});
	INFO("linked fragment should be discoverable");
	CHECK(bFoundShared);
	INFO("owned fragment should be independent from linked fragment");
	CHECK_FALSE(bAnyMatch);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Query::IndirectFragmentAccess", "[Mass][Query]")
{
	REQUIRE(EntityManager);
	const int32 IntValue = 1234;
	const float FloatValue = 10.0f;
	const int32 EntityCount = 100;

	TArray<FMassEntityHandle> IntEntities;
	EntityManager->BatchCreateEntities(IntsArchetype, EntityCount, IntEntities);
	for (FMassEntityHandle& Entity : IntEntities)
	{
		FTestFragment_Int& IntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity);
		IntFragment.Value = IntValue;
	}

	TArray<FMassEntityHandle> FloatEntities;
	EntityManager->BatchCreateEntities(FloatsArchetype, EntityCount, FloatEntities);
	int32 HandleIndex = 0;
	for (FMassEntityHandle& Entity : FloatEntities)
	{
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_EntityHandle::StaticStruct());
		FTestFragment_Float& FloatFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		FTestFragment_EntityHandle& EntityhandleFragment = EntityManager->GetFragmentDataChecked<FTestFragment_EntityHandle>(Entity);
		FloatFragment.Value = FloatValue;
		EntityhandleFragment.Value = IntEntities[HandleIndex++];
	}

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_EntityHandle>(EMassFragmentAccess::ReadOnly);
	Query.AddIndirectFragmentRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	Query.CacheArchetypes();

	bool bAllMatch = true;
	bool bFoundIndirect = true;
	FMassExecutionContext ExecContext(*EntityManager.Get());
	Query.ForEachEntityChunk(ExecContext, [&bAllMatch, &bFoundIndirect](FMassExecutionContext& Context)
		{
			const TConstArrayView<FTestFragment_EntityHandle> HandleFragments = Context.GetFragmentView<FTestFragment_EntityHandle>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FTestFragment_EntityHandle& HandleFragment = HandleFragments[EntityIt];
				const FTestFragment_Int* IntFragmentPtr = Context.GetIndirectConstFragmentPtr<FTestFragment_Int>(HandleFragment.Value);
				if (IntFragmentPtr)
				{
					bAllMatch |= IntFragmentPtr->Value == IntValue;
				}
				else
				{
					bFoundIndirect = false;
				}
			}
		});
	INFO("Indirect fragment should be discoverable");
	CHECK(bFoundIndirect);
	INFO("Indirect fragment should have correct value");
	CHECK(bAllMatch);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
