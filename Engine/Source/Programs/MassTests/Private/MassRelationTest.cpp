// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityBuilder.h"
#include "MassRelationManager.h"

#include "TestHarness.h"
#include "TestMacros/Assertions.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

using namespace UE::Mass;
using namespace UE::Mass::Relations;

//-----------------------------------------------------------------------------
// Relation test base — mirrors the original FRelationTestBase
//-----------------------------------------------------------------------------
struct FMassLLTRelationFixture : FMassLLTEntityFixture
{
	virtual ~FMassLLTRelationFixture() = default;

	UScriptStruct* MyRelationType = nullptr;
	const FRelationTypeTraits* RelationTraits = nullptr;
	FTypeHandle RelationTypeHandle;
	FRelationManager* RelationshipManagerPtr = nullptr;

	static TNotNull<UScriptStruct*> MakeRelationType(FName TypeName)
	{
		UScriptStruct* NewStruct = NewObject<UScriptStruct>(UScriptStruct::StaticClass(), FName(*FString::Printf(TEXT("Relation_%s"), *TypeName.ToString())), RF_Public);
		NewStruct->SetSuperStruct(FMassRelation::StaticStruct());
		return NewStruct;
	}

	template<UE::Mass::CElement TBase>
	static TNotNull<UScriptStruct*> MakeElementType(FName TypeName)
	{
		UScriptStruct* NewStruct = NewObject<UScriptStruct>(UScriptStruct::StaticClass(), FName(*FString::Printf(TEXT("%s"), *TypeName.ToString())), RF_Public);
		NewStruct->SetSuperStruct(TBase::StaticStruct());
		return NewStruct;
	}

	static FRelationTypeTraits CreateRelationTypeDescription(const FName FragmentName)
	{
		UScriptStruct* FragmentType = MakeRelationType(FragmentName);
		return CreateRelationTypeDescription(FragmentType);
	}

	static FRelationTypeTraits CreateRelationTypeDescription(TNotNull<UScriptStruct*> RelationType)
	{
		FRelationTypeTraits LocalRelationTraits(RelationType);
		LocalRelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Object)].DestructionPolicy = ERemovalPolicy::Destroy;
		LocalRelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Object)].bExclusive = true;
		LocalRelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::CleanUp;
		LocalRelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Subject)].bExclusive = false;
		LocalRelationTraits.bHierarchical = true;
		return LocalRelationTraits;
	}

	FMassLLTRelationFixture()
	{
		MyRelationType = MakeRelationType("MyChildOfRelation");
		FRelationTypeTraits TempRelationTraits = CreateRelationTypeDescription("MyChildOfRelation");
		TweakRelation(TempRelationTraits);
		RelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(TempRelationTraits));
		RelationTraits = &EntityManager->GetTypeManager().GetRelationTypeChecked(RelationTypeHandle);
		RelationshipManagerPtr = &EntityManager->GetRelationManager();
	}

	virtual void TweakRelation(FRelationTypeTraits& TempRelationTraits)
	{
	}

	FMassEntityHandle GetRelationSubject(const FMassEntityHandle ObjectEntity) const
	{
		REQUIRE(RelationshipManagerPtr);
		TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ObjectEntity);
		return Subjects.Num() ? Subjects[0] : FMassEntityHandle();
	}

	FMassEntityHandle GetRelationObject(const FMassEntityHandle SubjectEntity) const
	{
		REQUIRE(RelationshipManagerPtr);
		TArray<FMassEntityHandle> Objects = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, SubjectEntity);
		return Objects.Num() ? Objects[0] : FMassEntityHandle();
	}
};

//-----------------------------------------------------------------------------
// Symmetric cleanup fixture — overrides TweakRelation
//-----------------------------------------------------------------------------
struct FMassLLTSymmetricCleanupFixture : FMassLLTRelationFixture
{
	virtual void TweakRelation(FRelationTypeTraits& TempRelationTraits) override
	{
		TempRelationTraits.RoleTraits[0].DestructionPolicy = ERemovalPolicy::CleanUp;
		TempRelationTraits.RoleTraits[1].DestructionPolicy = ERemovalPolicy::CleanUp;
	}
};

//-----------------------------------------------------------------------------
// Complex hierarchy fixture
//-----------------------------------------------------------------------------
struct FMassLLTComplexHierarchyFixture : FMassLLTRelationFixture
{
	int32 NoRelation = INDEX_NONE;
	int32 SingleChildParent = INDEX_NONE;
	int32 SingleChildChild = INDEX_NONE;
	static constexpr int32 NumEntities = 9;
	bool Result[NumEntities][NumEntities] = {
		{ false, false, false, false, false, false, false, false, false },
		{ false, false, false, false, false, false, false, false, false },
		{ false, false, false, false, false, false, false, false, false },
		{ false, false, false, false, false, false, false, false, false },
		{ false, false, false, false, false, false, false, false, false },
		{ false, false, false, false, false, false, false, false, false },
		{ false, false, false, false, false, false, false, false, false },
		{ false, false, false, false, false, false, false, false, false },
		{ false, false, false, false, false, false, false, false, false },
	};

	FMassArchetypeHandle OriginalArchetype;
	TArray<FMassEntityHandle> Entities;

	FMassLLTComplexHierarchyFixture()
	{
		OriginalArchetype = IntsArchetype;
		EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

		RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[3], Entities[0]);
		Result[3][0] = true;
		RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[5], Entities[3]);
		Result[5][3] = true;
		Result[5][0] = true; // grandparent
		RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[7], Entities[3]);
		Result[7][3] = true;
		Result[7][0] = true; // grandparent
		RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[6], Entities[5]);
		Result[6][5] = true;
		Result[6][3] = true; // grandparent
		Result[6][0] = true; // grand-grandparent

		RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[8], Entities[0]);
		Result[8][0] = true;

		RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[4], Entities[1]);
		Result[4][1] = true;

		NoRelation = 2;
		SingleChildParent = 1;
		SingleChildChild = 4;

		// we now have the following hierarchy, indexing Entities array
		// 0				1				2 - Not in hierarchy
		// +---	3			+- 4
		// |	+ 	5
		// |	|	+	 6
		// |	|
		// |	+ 	7
		// |
		// +---	8
		//
	}
};

//-----------------------------------------------------------------------------
// Tests
//-----------------------------------------------------------------------------

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::CreateSingle", "[Mass][Relation]")
{
	FMassArchetypeHandle OriginalArchetype = IntsArchetype;
	TArray<FMassEntityHandle> CreatedEntities;
	EntityManager->BatchCreateEntities(OriginalArchetype, 2, CreatedEntities);

	const FMassEntityHandle ChildEntity = CreatedEntities[0];
	const FMassEntityHandle ParentEntity = CreatedEntities[1];

	FMassEntityHandle CreatedRelationEntity = RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, ChildEntity, ParentEntity);
	EntityManager->FlushCommands();

	INFO("Valid relation entity has been created");
	CHECK(CreatedRelationEntity.IsValid());

	TArray<FMassEntityHandle> TestedEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ChildEntity);
	INFO("No subjects point to the ChildEntity as the relation's object");
	CHECK(TestedEntities.IsEmpty());
	TestedEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ParentEntity);
	INFO("ParentEntity is an object of a relation of the given type");
	CHECK(TestedEntities.Num() > 0);
	INFO("ParentEntity is an object of exactly one relation of the given type");
	REQUIRE(TestedEntities.Num() == 1);
	INFO("The ChildEntity is the subject of the give relation, where ParentEntity is the object");
	CHECK(TestedEntities[0] == ChildEntity);

	TestedEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, ChildEntity);
	INFO("The ChildEntity is has exactly one object for the given relation type");
	REQUIRE(TestedEntities.Num() == 1);
	INFO("The object for the given relation for the ChildEntity is the ParentEntity");
	CHECK(TestedEntities[0] == ParentEntity);
	TestedEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, ParentEntity);
	INFO("The ParentEntity has no objects for the given relation type");
	CHECK(TestedEntities.IsEmpty());

	RelationshipManagerPtr->DestroyRelationInstance(RelationTypeHandle, ChildEntity, ParentEntity);
	EntityManager->FlushCommands();
	TestedEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ChildEntity);
	INFO("No relation subjects for ChildEntity still");
	CHECK(TestedEntities.IsEmpty());
	TestedEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ParentEntity);
	INFO("No relation subjects for ParentEntity anymore");
	CHECK(TestedEntities.IsEmpty());

	TestedEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, ChildEntity);
	INFO("No relation objects for ChildEntity anymore");
	CHECK(TestedEntities.IsEmpty());
	TestedEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, ParentEntity);
	INFO("No relation objects for ParentEntity still");
	CHECK(TestedEntities.IsEmpty());
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::Hierarchy::Unit", "[Mass][Relation]")
{
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(OriginalArchetype, 2, Entities);

	constexpr int32 ParentIndex = 1;
	constexpr int32 ChildIndex = 0;
	RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[ChildIndex], Entities[ParentIndex]);

	const FMassArchetypeHandle ArchetypeParent = EntityManager->GetArchetypeForEntity(Entities[ParentIndex]);
	const FMassArchetypeHandle ArchetypeChild = EntityManager->GetArchetypeForEntity(Entities[ChildIndex]);

	INFO("Child and Parent archetypes");
	CHECK(ArchetypeParent != ArchetypeChild);
	INFO("Child and the original archetype");
	CHECK(ArchetypeChild != OriginalArchetype);
	INFO("Parent and the original archetype");
	CHECK(ArchetypeParent != OriginalArchetype);

	FArchetypeGroupHandle GroupHandle = EntityManager->GetGroupForEntity(Entities[ChildIndex], RelationTraits->RegisteredGroupType);
	INFO("Leaf entity's is at expected group level");
	CHECK(static_cast<int32>(GroupHandle.GetGroupID()) == 1);
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::Hierarchy::Chain", "[Mass][Relation]")
{
	constexpr int32 NumEntities = 3;
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

	constexpr int32 LeafIndex = 1;
	constexpr int32 MiddleIndex = 0;
	constexpr int32 RootIndex = 2;
	RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[LeafIndex], Entities[MiddleIndex]);
	RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[MiddleIndex], Entities[RootIndex]);

	FMassArchetypeHandle Archetypes[NumEntities] = {
		EntityManager->GetArchetypeForEntity(Entities[0])
		, EntityManager->GetArchetypeForEntity(Entities[1])
		, EntityManager->GetArchetypeForEntity(Entities[2])
	};

	for (int32 Index = 0; Index < NumEntities; ++Index)
	{
		INFO("Each pair of archetypes");
		CHECK(Archetypes[Index] != Archetypes[(Index + 1) % NumEntities]);
	}
	{
		constexpr int32 EntityIndex = LeafIndex;
		const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);

		INFO("Leaf entity's Object vs Middle");
		CHECK(Object == Entities[MiddleIndex]);
		TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Entities[EntityIndex]);
		INFO("Leaf entity's has no children");
		CHECK(Subjects.IsEmpty());

		FArchetypeGroupHandle GroupHandle = EntityManager->GetGroupForEntity(Entities[EntityIndex], RelationTraits->RegisteredGroupType);
		INFO("Leaf entity's is at expected group level");
		CHECK(static_cast<int32>(GroupHandle.GetGroupID()) == 2);
	}
	{
		constexpr int32 EntityIndex = MiddleIndex;
		const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
		INFO("Middle entity's Object vs Root");
		CHECK(Object == Entities[RootIndex]);
		TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Entities[EntityIndex]);
		INFO("Middle entity's has a single child");
		CHECK(Subjects.Num() == 1);
		INFO("Middle entity's sole child is the leaf entity");
		CHECK(Subjects.Last() == Entities[LeafIndex]);

		FArchetypeGroupHandle GroupHandle = EntityManager->GetGroupForEntity(Entities[EntityIndex], RelationTraits->RegisteredGroupType);
		INFO("Leaf entity's is at expected group level");
		CHECK(static_cast<int32>(GroupHandle.GetGroupID()) == 1);
	}
	{
		constexpr int32 EntityIndex = RootIndex;
		const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
		INFO("Root entity has no parent");
		CHECK(Object.IsValid() == false);
		TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Entities[EntityIndex]);
		INFO("Root entity's has a single child");
		CHECK(Subjects.Num() == 1);
		INFO("Root entity's sole child is the middle entity");
		CHECK(Subjects.Last() == Entities[MiddleIndex]);

		FArchetypeGroupHandle GroupHandle = EntityManager->GetGroupForEntity(Entities[EntityIndex], RelationTraits->RegisteredGroupType);
		INFO("Leaf entity's is at expected group level");
		CHECK(static_cast<int32>(GroupHandle.GetGroupID()) == 0);
	}
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::Hierarchy::TrivialReparenting", "[Mass][Relation]")
{
	if (FEntityBuilder::GetDefaultValueOfForceDeferredCommit())
	{
		// this test assumes synchronous EntityBuilder commits. Bailing out
		// to avoid generating false positives
		return;
	}

	constexpr int32 NumEntities = 3;
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

	constexpr int32 LeafIndex = 1;
	constexpr int32 OriginalParentIndex = 0;
	constexpr int32 FinalParentIndex = 2;
	RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[LeafIndex], Entities[OriginalParentIndex]);
	RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[LeafIndex], Entities[FinalParentIndex]);
	{
		constexpr int32 EntityIndex = LeafIndex;
		const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
		INFO("Leaf entity's Object vs Original Parent");
		CHECK(Object != Entities[OriginalParentIndex]);
		INFO("Leaf entity's Object vs Final Parent");
		CHECK(Object == Entities[FinalParentIndex]);

		TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Entities[EntityIndex]);
		INFO("Leaf entity's has no children");
		CHECK(Subjects.IsEmpty());
	}
	{
		constexpr int32 EntityIndex = OriginalParentIndex;
		const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
		INFO("Original Parent has a parent");
		CHECK_FALSE(Object.IsValid());

		TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Entities[EntityIndex]);
		INFO("Original Parent has no children");
		CHECK(Subjects.IsEmpty());
	}
	{
		constexpr int32 EntityIndex = FinalParentIndex;
		const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
		INFO("Final Parent has a parent");
		CHECK_FALSE(Object.IsValid());

		TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Entities[EntityIndex]);
		INFO("Final Parent has one child");
		CHECK(Subjects.Num() == 1);
		INFO("Final Parent's sole child is the Leaf entity");
		CHECK(Subjects.Last() == Entities[LeafIndex]);
	}
}

TEST_CASE_METHOD(FMassLLTComplexHierarchyFixture, "Mass::Relation::Hierarchy::ChildOf", "[Mass][Relation]")
{
	// no relation
	for (int32 EntityIndex = 0; EntityIndex < Entities.Num(); ++EntityIndex)
	{
		if (EntityIndex != NoRelation)
		{
			INFO("NoRelation is a child of another entity");
			CHECK_FALSE(RelationshipManagerPtr->IsSubjectOfRelation(RelationTypeHandle, Entities[NoRelation], Entities[EntityIndex]));
			INFO("NoRelation is a parent to another entity");
			CHECK_FALSE(RelationshipManagerPtr->IsSubjectOfRelation(RelationTypeHandle, Entities[EntityIndex], Entities[NoRelation]));
		}
	}

	// simple 1-1 hierarchy
	INFO("(NOT) SingleChildParent is a child of SingleChildChild");
	CHECK_FALSE(RelationshipManagerPtr->IsSubjectOfRelation(RelationTypeHandle, Entities[SingleChildParent], Entities[SingleChildChild]));
	INFO("SingleChildChild is a child of SingleChildParent");
	CHECK(RelationshipManagerPtr->IsSubjectOfRelation(RelationTypeHandle, Entities[SingleChildChild], Entities[SingleChildParent]));

	// We'll test everything else in bulk. Here's what we want to see, rows are "children", columns the result of IsSubjectOfRelation
	// note that Result contains cumulative results, so we test indirect relations too
	for (int32 ChildIndex = 0; ChildIndex < Entities.Num(); ++ChildIndex)
	{
		for (int32 ParentIndex = 0; ParentIndex < Entities.Num(); ++ParentIndex)
		{
			INFO("Child is a child");
			CHECK(RelationshipManagerPtr->IsSubjectOfRelationRecursive(RelationTypeHandle, Entities[ChildIndex], Entities[ParentIndex]) == Result[ChildIndex][ParentIndex]);
		}
	}
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::Policy::OnObjectDestroyed", "[Mass][Relation]")
{
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

	const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
	const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
	RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, SubjectEntity, ObjectEntity);

	EntityManager->DestroyEntity(ObjectEntity);
	EntityManager->FlushCommands();
	INFO("The source entity is destroyed along with the relation object");
	CHECK(EntityManager->IsEntityValid(SubjectEntity) == false);
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::Policy::OnSubjectDestroyed", "[Mass][Relation]")
{
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;
	{
		FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestRelationA");
		LocalRelationTraits.RoleTraits[static_cast<int32>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::Destroy;

		const FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

		const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, SubjectEntity, ObjectEntity);

		EntityManager->DestroyEntity(SubjectEntity);
		EntityManager->FlushCommands();
		INFO("The object entity is destroyed along with the relation source");
		CHECK(EntityManager->IsEntityValid(ObjectEntity) == false);
	}
	{
		FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestRelationB");
		LocalRelationTraits.RoleTraits[static_cast<int32>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::CleanUp;
		const FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

		const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, SubjectEntity, ObjectEntity);

		EntityManager->DestroyEntity(SubjectEntity);
		EntityManager->FlushCommands();
		INFO("The object entity is still valid once the relation source gets destroyed");
		CHECK(EntityManager->IsEntityValid(ObjectEntity));
		// The relation has been cleaned up
		INFO("The object entity is still valid once the the relation source gets destroyed");
		CHECK(EntityManager->IsEntityValid(ObjectEntity));
		TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, ObjectEntity);
		INFO("Expected remaining sources count");
		CHECK(Subjects.Num() == 0);
	}
	{
		FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestRelationC");
		LocalRelationTraits.RoleTraits[static_cast<int32>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::Splice;
		// not that for this test to work the Object's Destruction policy can't be Destroy
		// since then the LeafEntity (created below) will be destroyed as part of SubjectEntity's destruction
		// before the new, patched relation gets created
		LocalRelationTraits.RoleTraits[static_cast<int32>(ERelationRole::Object)].DestructionPolicy = ERemovalPolicy::Splice;
		const FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

		const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		const FMassEntityHandle LeafEntity = EntityManager->CreateEntity(OriginalArchetype);
		const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, SubjectEntity, ObjectEntity);
		RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, LeafEntity, SubjectEntity);

		EntityManager->DestroyEntity(SubjectEntity);
		EntityManager->FlushCommands();
		INFO("The object entity is still valid once the relation source gets destroyed");
		CHECK(EntityManager->IsEntityValid(ObjectEntity));
		INFO("The leaf entity is still valid once the relation source gets destroyed");
		CHECK(EntityManager->IsEntityValid(LeafEntity));
		TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, ObjectEntity);
		INFO("Expected remaining sources count");
		REQUIRE(Subjects.Num() == 1);
		INFO("The leaf entity is the child now");
		CHECK(Subjects[0] == LeafEntity);
	}
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::Policy::DestroyEverything", "[Mass][Relation]")
{
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;
	{
		const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, SubjectEntity, ObjectEntity);

		EntityManager->DestroyEntity(ObjectEntity);
		EntityManager->FlushCommands();
		INFO("The source entity is destroyed along with the relation object");
		CHECK(EntityManager->IsEntityValid(SubjectEntity) == false);
	}
	{
		const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
		RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, SubjectEntity, ObjectEntity);

		EntityManager->DestroyEntity(SubjectEntity);
		EntityManager->FlushCommands();
		INFO("The source entity is destroyed along with the relation object");
		CHECK(EntityManager->IsEntityValid(SubjectEntity) == false);
	}
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::Hierarchy::TrivialCycle", "[Mass][Relation]")
{
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

	const FMassEntityHandle Entity = EntityManager->CreateEntity(OriginalArchetype);
	// ensureMsgf at MassRelationManager.cpp:182 fires only once per process (one-time guard).
	// InvalidRelations test typically consumes it first. Use FEnsureScope to catch if it does fire.
	{
		FEnsureScope EnsureScope;
		RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entity, Entity);
	}

	const FMassArchetypeHandle FinalArchetype = EntityManager->GetArchetypeForEntity(Entity);
	INFO("The entity has not changed archetypes");
	CHECK(FinalArchetype == OriginalArchetype);
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::DestructionPolicy", "[Mass][Relation]")
{
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

	UScriptStruct* DestroyChildOnParentDestructionRelation = MakeRelationType("DestroyChildOnParentDestruction");
	FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription(DestroyChildOnParentDestructionRelation);
	LocalRelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Object)].DestructionPolicy = ERemovalPolicy::Destroy;

	FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(OriginalArchetype, 2, Entities);

	RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, Entities[1], Entities[0]);
	EntityManager->FlushCommands();

	EntityManager->DestroyEntity(Entities[0]);

	EntityManager->FlushCommands();
	INFO("Destroying the parent destroys the child.");
	CHECK(EntityManager->IsEntityActive(Entities[1]) == false);
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::Exclusivity", "[Mass][Relation]")
{
	if (FEntityBuilder::GetDefaultValueOfForceDeferredCommit())
	{
		// this test assumes synchronous EntityBuilder commits. Bailing out
		// to avoid generating false positives
		return;
	}

	constexpr int32 NumEntities = 3;
	constexpr int32 OriginalParentIndex = 0;
	constexpr int32 ChildIndex = 1;
	constexpr int32 NewParentIndex = 2;
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

	UScriptStruct* DestroyChildOnParentDestructionRelation = MakeRelationType("DestroyChildOnParentDestruction");
	FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription(DestroyChildOnParentDestructionRelation);
	LocalRelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::Destroy;

	FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

	RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, Entities[ChildIndex], Entities[OriginalParentIndex]);
	// swapping parent
	RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, Entities[ChildIndex], Entities[NewParentIndex]);

	INFO("Original parent has no children");
	CHECK(RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, Entities[OriginalParentIndex]).Num() == 0);
	INFO("New parent has exactly one child");
	REQUIRE(RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, Entities[NewParentIndex]).Num() == 1);
	INFO("New parent's child matches expectations");
	CHECK(RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, Entities[NewParentIndex])[0] == Entities[ChildIndex]);
	INFO("The Child has parents");
	REQUIRE(RelationshipManagerPtr->GetRelationObjects(LocalRelationTypeHandle, Entities[ChildIndex]).Num() > 0);
	INFO("The Child knows about the new parent");
	CHECK(RelationshipManagerPtr->GetRelationObjects(LocalRelationTypeHandle, Entities[ChildIndex])[0] == Entities[NewParentIndex]);
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::InvalidRelations", "[Mass][Relation]")
{
	if (FEntityBuilder::GetDefaultValueOfForceDeferredCommit())
	{
		// this test assumes synchronous EntityBuilder commits. Bailing out
		// to avoid generating false positives
		return;
	}

	constexpr int32 NumEntities = 3;
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

	UScriptStruct* DestroyChildOnParentDestructionRelation = MakeRelationType("DestroyChildOnParentDestruction");
	FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription(DestroyChildOnParentDestructionRelation);
	const FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

	RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, Entities[0], Entities[1]);
	EntityManager->FlushCommands();

	// try creating relations with mismatching pairs
	{
		FEnsureScope EnsureScope;
		TArray<FMassEntityHandle> RelationInstances = RelationshipManagerPtr->CreateRelationInstances(LocalRelationTypeHandle
			, TArrayView<FMassEntityHandle>(&Entities[0], 1), TArrayView<FMassEntityHandle>(&Entities[1], 2));
		CHECK(EnsureScope.GetCount() > 0);
		INFO("A: Mismatching relation instances created count");
		CHECK(RelationInstances.Num() == 0);
	}
	{
		// Multiple expected ensure failures: invalid subject, invalid object, self-relation, duplicate relation
		FEnsureScope EnsureScope;

		TArray<FMassEntityHandle> Subjects;
		TArray<FMassEntityHandle> Objects;

		// Invalid subject entity
		Subjects.Add(FMassEntityHandle());
		Objects.Add(Entities[0]);

		// Invalid object entity
		Subjects.Add(Entities[0]);
		Objects.Add(FMassEntityHandle());

		// Self-relation (subject == object)
		Subjects.Add(Entities[0]);
		Objects.Add(Entities[0]);

		// this one we expect to succeed:
		Subjects.Add(Entities[1]);
		Objects.Add(Entities[2]);

		// Duplicate relation
		Subjects.Add(Entities[0]);
		Objects.Add(Entities[1]);

		TArray<FMassEntityHandle> RelationInstances = RelationshipManagerPtr->CreateRelationInstances(LocalRelationTypeHandle, Subjects, Objects);
		CHECK(EnsureScope.GetCount() > 0);

		INFO("Expected number of valid relation instances created");
		REQUIRE(RelationInstances.Num() == 1);

		FMassRelationFragment* RelationFragment = EntityManager->GetFragmentDataPtr<FMassRelationFragment>(RelationInstances[0]);
		INFO("The created relation entity has the relation fragment");
		REQUIRE(RelationFragment != nullptr);
		INFO("The created relation instance has the expected subject");
		CHECK(RelationFragment->Subject == Entities[1]);
		INFO("The created relation instance has the expected object");
		CHECK(RelationFragment->Object == Entities[2]);
	}
}

TEST_CASE_METHOD(FMassLLTRelationFixture, "Mass::Relation::Type::Registration", "[Mass][Relation]")
{
	{
		FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestFragmentA");
		EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));
	}
	{
		FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestFragmentA");
		// make it different
		LocalRelationTraits.bHierarchical = !LocalRelationTraits.bHierarchical;
		REQUIRE_ENSURE_MSG("Modifying relationship after registration done is not supported",
			EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits)));
	}
	{
		FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestFragmentB");
		// register with existing name — ensure suppressed by ensureMsgf one-time-per-callsite guard
		// (the ensure at MassTypeManager.cpp already fired above)
		FEnsureScope EnsureScope;
		EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));
	}
	{
		FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestFragmentA");
		// register existing traits — ensure suppressed (same callsite, same one-time guard)
		FEnsureScope EnsureScope;
		EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));
	}
}

TEST_CASE_METHOD(FMassLLTSymmetricCleanupFixture, "Mass::Relation::SymmetricCleanup", "[Mass][Relation]")
{
	const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(OriginalArchetype, 2, Entities);

	constexpr int32 ObjectIndex = 1;
	constexpr int32 SubjectIndex = 0;
	RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[SubjectIndex], Entities[ObjectIndex]);

	TArray<FMassEntityHandle> SubjectEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Entities[ObjectIndex]);
	INFO("Expected number of object's subjects");
	REQUIRE(SubjectEntities.Num() == 1);
	INFO("Object's subject meets expectations");
	CHECK(SubjectEntities[0] == Entities[SubjectIndex]);

	TArray<FMassEntityHandle> ObjectEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, Entities[SubjectIndex]);
	INFO("Expected number of subject's objects");
	REQUIRE(ObjectEntities.Num() == 1);
	INFO("Subject's object meets expectations");
	CHECK(ObjectEntities[0] == Entities[ObjectIndex]);

	EntityManager->DestroyEntity(Entities[ObjectIndex]);

	ObjectEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, Entities[SubjectIndex]);
	INFO("After object's destruction: Expected number of subject's objects");
	CHECK(ObjectEntities.Num() == 0);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
