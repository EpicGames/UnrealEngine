// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Serialization::BitSet::Trivial", "[Mass][Serialization]")
{
	FMassFragmentBitSet BitSet;

	BitSet.Add<FTestFragment_Float>();
	BitSet.Add<FTestFragment_Bool>();
	BitSet.Add<FTestFragment_Int>();

	TArray<uint8> Data;
	FMemoryWriter Writer(Data);

	BitSet.Serialize(Writer);

	FMemoryReader Reader(Data);
	FMassFragmentBitSet NewBitSet;
	NewBitSet.Serialize(Reader);

	CHECK(BitSet.IsEquivalent(NewBitSet));
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Serialization::BitSet::OverrideExisting", "[Mass][Serialization]")
{
	FMassFragmentBitSet RegularBitSet;
	RegularBitSet.Add<FTestFragment_Float>();
	RegularBitSet.Add<FTestFragment_Int>();

	TArray<uint8> Data;
	FMemoryWriter Writer(Data);
	RegularBitSet.Serialize(Writer);

	FMemoryReader Reader(Data);
	FMassFragmentBitSet SerializedBitSet;
	// pollute the bitset
	SerializedBitSet.Add<FTestFragment_Large>();
	SerializedBitSet.Add<FTestFragment_Array>();

	SerializedBitSet.Serialize(Reader);

	CHECK(RegularBitSet.IsEquivalent(SerializedBitSet));
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
