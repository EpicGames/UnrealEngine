// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassArchetypeData.h"
#include "MassArchetypeTypes.h"
#include "Algo/RandomShuffle.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

//-----------------------------------------------------------------------------
// Helpers
//-----------------------------------------------------------------------------
template<typename TMassSharedFragmentType>
const TMassSharedFragmentType* GetConstSharedFragmentPtr(const FMassArchetypeSharedFragmentValues& Values)
{
	FConstSharedStruct SharedStruct = Values.GetConstSharedFragmentStruct(TMassSharedFragmentType::StaticStruct());
	return SharedStruct.IsValid() ? SharedStruct.GetPtr<const TMassSharedFragmentType>() : nullptr;
}

template<typename TMassSharedFragmentType>
TMassSharedFragmentType* GetMutableSharedFragmentPtr(FMassArchetypeSharedFragmentValues& Values)
{
	FSharedStruct SharedStruct = Values.GetSharedFragmentStruct(TMassSharedFragmentType::StaticStruct());
	return SharedStruct.IsValid() ? SharedStruct.GetPtr<TMassSharedFragmentType>() : nullptr;
}

bool HasExactSharedFragmentTypesMatch(const FMassArchetypeSharedFragmentValues& Values, const FMassSharedFragmentBitSet SharedFragmentBitSet)
{
	return Values.GetBitSet().Get<FMassSharedFragmentBitSet>() == SharedFragmentBitSet;
}

bool HasExactConstSharedFragmentTypesMatch(const FMassArchetypeSharedFragmentValues& Values, const FMassConstSharedFragmentBitSet SharedFragmentBitSet)
{
	return Values.GetBitSet().Get<FMassConstSharedFragmentBitSet>() == SharedFragmentBitSet;
}

//-----------------------------------------------------------------------------
// FMassLLTSharedFragmentFixture - extends entity fixture with shared fragment helpers
//-----------------------------------------------------------------------------
struct FMassLLTSharedFragmentFixture : FMassLLTEntityFixture
{
	template<typename TSharedStruct, typename TSharedFragment>
	FConstStructView GetSharedFragmentView(const FMassEntityHandle EntityHandle)
	{
		if constexpr (TIsDerivedFrom<TSharedStruct, FSharedStruct>::IsDerived)
		{
			return EntityManager->GetSharedFragmentDataStruct(EntityHandle, TSharedFragment::StaticStruct());
		}
		else
		{
			return EntityManager->GetConstSharedFragmentDataStruct(EntityHandle, TSharedFragment::StaticStruct());
		}
	}

	template<typename TSharedStruct, typename TSharedFragment>
	void CreateEntities(TArray<FMassEntityHandle>& OutEntityHandles, const int32 NumToCreate, const typename TSharedFragment::FValueType TestValue)
	{
		TSharedFragment FragmentInstance(TestValue);
		FMassArchetypeSharedFragmentValues SharedIntValues;

		TSharedStruct SharedFragmentInstance = TSharedStruct::Make(FragmentInstance);
		SharedIntValues.Add(SharedFragmentInstance);

		EntityManager->BatchCreateEntities(FloatsArchetype, SharedIntValues, NumToCreate, OutEntityHandles);
	}

	template<typename TSharedStruct, typename TSharedFragment>
	void CreateEntity(FMassEntityHandle& OutEntityHandle, const typename TSharedFragment::FValueType TestValue)
	{
		TSharedFragment FragmentInstance(TestValue);
		FMassArchetypeSharedFragmentValues SharedIntValues;

		TSharedStruct SharedFragmentInstance = TSharedStruct::Make(FragmentInstance);
		SharedIntValues.Add(SharedFragmentInstance);

		OutEntityHandle = EntityManager->CreateEntity(FloatsArchetype, SharedIntValues);
	}
};

//-----------------------------------------------------------------------------
// FMassArchetypeSharedFragmentValues Tests
//-----------------------------------------------------------------------------

TEST_CASE_METHOD(FMassLLTFixture, "Mass::SharedFragments::CreateValue", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 59;
	FMassArchetypeSharedFragmentValues Values;

	{
		FTestSharedFragment_Int FragmentInstance(TestIntValue);
		FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
		Values.Add(SharedFragmentInstance);
	}

	const FTestSharedFragment_Int* ConstInstance = GetConstSharedFragmentPtr<FTestSharedFragment_Int>(Values);
	FTestSharedFragment_Int* NonConstInstance = GetMutableSharedFragmentPtr<FTestSharedFragment_Int>(Values);

	INFO("Fetching fragment as a const shared fragment should fail");
	CHECK(ConstInstance == nullptr);
	INFO("Fetching fragment as a shared fragment should not fail");
	REQUIRE(NonConstInstance != nullptr);
	INFO("The fetched value should match the expectations");
	CHECK(NonConstInstance->Value == TestIntValue);
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::SharedFragments::CreateConstValue", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 59;
	FMassArchetypeSharedFragmentValues Values;

	{
		FConstSharedStruct SharedFragmentInstance = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue);
		Values.Add(SharedFragmentInstance);
	}

	const FTestConstSharedFragment_Int* ConstInstance = GetConstSharedFragmentPtr<FTestConstSharedFragment_Int>(Values);
	FTestConstSharedFragment_Int* NonConstInstance = GetMutableSharedFragmentPtr<FTestConstSharedFragment_Int>(Values);

	INFO("Fetching fragment as a shared fragment should fail");
	CHECK(NonConstInstance == nullptr);
	INFO("Fetching fragment as a const shared fragment should not fail");
	REQUIRE(ConstInstance != nullptr);
	INFO("The fetched value should match the expectations");
	CHECK(ConstInstance->Value == TestIntValue);
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::SharedFragments::Contains", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 31;
	constexpr float TestFloatValue = 63.f;
	FMassArchetypeSharedFragmentValues Values;

	INFO("Empty FMassArchetypeSharedFragmentValues should fail ContainsType tests (StaticStruct)");
	CHECK_FALSE(Values.ContainsType(FTestSharedFragment_Int::StaticStruct()));
	INFO("Empty FMassArchetypeSharedFragmentValues should fail ContainsType tests (template)");
	CHECK_FALSE(Values.ContainsType<FTestSharedFragment_Int>());

	{
		FSharedStruct SharedFragmentInstance = FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue);
		Values.Add(SharedFragmentInstance);
	}

	INFO("After adding Int, ContainsType with StaticStruct should succeed");
	CHECK(Values.ContainsType(FTestSharedFragment_Int::StaticStruct()));
	INFO("After adding Int, ContainsType with template should succeed");
	CHECK(Values.ContainsType<FTestSharedFragment_Int>());

	{
		FSharedStruct SharedFragmentInstance = FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue);
		Values.Add(SharedFragmentInstance);
	}
	INFO("After adding Float, ContainsType with StaticStruct should succeed");
	CHECK(Values.ContainsType(FTestSharedFragment_Float::StaticStruct()));
	INFO("After adding Float, ContainsType with template should succeed");
	CHECK(Values.ContainsType<FTestSharedFragment_Float>());
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::SharedFragments::Append", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 31;
	constexpr float TestFloatValue = 63.f;

	FMassArchetypeSharedFragmentValues ValuesNonConstInt;
	ValuesNonConstInt.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
	FMassArchetypeSharedFragmentValues ValuesNonConstFloat;
	ValuesNonConstFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));
	FMassArchetypeSharedFragmentValues ValuestNonConstIntFloat;
	ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
	ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));

	// Appending int/float values to a new Values instance should result in the same result as Adding them
	{
		FMassArchetypeSharedFragmentValues Values;
		Values.Append(ValuesNonConstInt);
		INFO("#1 Append results should match expectations");
		CHECK(Values.HasSameValues(ValuesNonConstInt));
		Values.Append(ValuesNonConstFloat);
		INFO("#2 Append results should match expectations");
		CHECK(Values.HasSameValues(ValuestNonConstIntFloat));
	}
	{
		FMassArchetypeSharedFragmentValues Values;
		Values.Append(ValuesNonConstFloat);
		INFO("#3 Append results should match expectations");
		CHECK(Values.HasSameValues(ValuesNonConstFloat));
		Values.Append(ValuesNonConstInt);
		INFO("#4 Append results should match expectations");
		CHECK(Values.HasSameValues(ValuestNonConstIntFloat));
	}

	FMassArchetypeSharedFragmentValues ValuesConstInt;
	ValuesConstInt.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
	FMassArchetypeSharedFragmentValues ValuesConstFloat;
	ValuesConstFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(TestFloatValue));
	FMassArchetypeSharedFragmentValues ValuestConstIntFloat;
	ValuestConstIntFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
	ValuestConstIntFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(TestFloatValue));

	{
		FMassArchetypeSharedFragmentValues Values;
		Values.Append(ValuesConstInt);
		INFO("#5 Append results should match expectations");
		CHECK(Values.HasSameValues(ValuesConstInt));
		Values.Append(ValuesConstFloat);
		INFO("#6 Append results should match expectations");
		CHECK(Values.HasSameValues(ValuestConstIntFloat));
	}
	{
		FMassArchetypeSharedFragmentValues Values;
		Values.Append(ValuesConstFloat);
		INFO("#7 Append results should match expectations");
		CHECK(Values.HasSameValues(ValuesConstFloat));
		Values.Append(ValuesConstInt);
		INFO("#8 Append results should match expectations");
		CHECK(Values.HasSameValues(ValuestConstIntFloat));
	}
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::SharedFragments::Remove", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 31;
	constexpr float TestFloatValue = 63.f;

	{
		FMassArchetypeSharedFragmentValues ValuesNonConstInt;
		ValuesNonConstInt.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
		FMassArchetypeSharedFragmentValues ValuesNonConstFloat;
		ValuesNonConstFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));
		FMassArchetypeSharedFragmentValues ValuestNonConstIntFloat;
		ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
		ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));

		{
			FMassArchetypeSharedFragmentValues Values = ValuestNonConstIntFloat;
			INFO("Assignment should result in same values");
			CHECK(Values.HasSameValues(ValuestNonConstIntFloat));

			// removing just the Int shared fragment
			Values.Remove(ValuesNonConstInt);
			INFO("#1 Removal results should match expectations");
			CHECK(Values.HasSameValues(ValuesNonConstFloat));
		}
		{
			FMassArchetypeSharedFragmentValues Values = ValuestNonConstIntFloat;
			// removing just the Float shared fragment
			Values.Remove(ValuesNonConstFloat);
			INFO("#2 Removal results should match expectations");
			CHECK(Values.HasSameValues(ValuesNonConstInt));
		}
	}
	{
		FMassArchetypeSharedFragmentValues ValuesConstInt;
		ValuesConstInt.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
		FMassArchetypeSharedFragmentValues ValuesConstFloat;
		ValuesConstFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(TestFloatValue));
		FMassArchetypeSharedFragmentValues ValuestConstIntFloat;
		ValuestConstIntFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
		ValuestConstIntFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(TestFloatValue));

		{
			FMassArchetypeSharedFragmentValues Values = ValuestConstIntFloat;
			INFO("Assignment should result in same values");
			CHECK(Values.HasSameValues(ValuestConstIntFloat));

			// removing just the Int shared fragment
			Values.Remove(ValuesConstInt);
			INFO("#3 Removal results should match expectations");
			CHECK(Values.HasSameValues(ValuesConstFloat));
		}
		{
			FMassArchetypeSharedFragmentValues Values = ValuestConstIntFloat;
			// removing just the Float shared fragment
			Values.Remove(ValuesConstFloat);
			INFO("#4 Removal results should match expectations");
			CHECK(Values.HasSameValues(ValuesConstInt));
		}
	}
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::SharedFragments::Hash", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 31;
	constexpr float TestFloatValue = 63.f;

	FMassArchetypeSharedFragmentValues ValuestNonConstIntFloat;
	ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
	ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));

	// Note: Calling CalculateHash() on unsorted collection intentionally triggers ensure(bSorted),
	// which is fatal in LLT. The original test verified it returns 0 via MASS_SCOPED_ENSURE_TEST.
	// Skipping the unsorted hash check here.

	ValuestNonConstIntFloat.Sort();
	const uint32 ValidHash = ValuestNonConstIntFloat.CalculateHash();
	INFO("Expecting sorted collection hashing to result in non 0");
	CHECK(ValidHash != 0u);
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::SharedFragments::ForEach", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumSharedFragments = 4;
	TStaticArray<int32, NumSharedFragments> TestInitValues = { 9, 1, 12, 13 };

	for (int32 InitValue : TestInitValues)
	{
		EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>(InitValue); //-V530
	}

	TArray<int32> Results;
	TArray<int32> ModifiedValues;
	EntityManager->ForEachSharedFragment<FTestSharedFragment_Int>([&Results, &ModifiedValues](FTestSharedFragment_Int& SharedFragment)
	{
		Results.Add(SharedFragment.Value);
		ModifiedValues.Add(SharedFragment.Value += 100);
	});

	INFO("Number of processed shared fragments");
	CHECK(Results.Num() == NumSharedFragments);
	for (int32 InitValue : TestInitValues)
	{
		INFO("Read values matches init values");
		CHECK(Results.Find(InitValue) != INDEX_NONE);
	}

	TArray<int32> MutatedResults;
	EntityManager->ForEachSharedFragment<FTestSharedFragment_Int>([&MutatedResults](const FTestSharedFragment_Int& SharedFragment)
	{
		MutatedResults.Add(SharedFragment.Value);
	});

	INFO("Number of shared fragments processed in second round");
	CHECK(MutatedResults.Num() == NumSharedFragments);
	for (int32 ModifiedValue : ModifiedValues)
	{
		INFO("Read values matches values set in the first round");
		CHECK(MutatedResults.Find(ModifiedValue) != INDEX_NONE);
	}

	constexpr int32 ConditionalLimit = 10;
	TArray<int32> ConditionalResults;
	EntityManager->ForEachSharedFragmentConditional<FTestSharedFragment_Int>(
		[](FTestSharedFragment_Int& SharedFragment)
		{
			return SharedFragment.Value > ConditionalLimit;
		}
		, [&ConditionalResults](FTestSharedFragment_Int& SharedFragment)
		{
			ConditionalResults.Add(SharedFragment.Value);
		}
	);
	for (int32 Value : ConditionalResults)
	{
		INFO("Only the values matching the condition get processed");
		CHECK(Value > ConditionalLimit);
	}
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::ConstSharedFragments::ForEach", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumSharedFragments = 4;
	TStaticArray<int32, NumSharedFragments> TestInitValues = { 9, 1, 12, 13 };

	for (int32 InitValue : TestInitValues)
	{
		EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>(InitValue); //-V530
	}

	TArray<int32> Results;
	EntityManager->ForEachConstSharedFragment<FTestConstSharedFragment_Int>([&Results](const FTestConstSharedFragment_Int& SharedFragment)
	{
		Results.Add(SharedFragment.Value);
	});

	INFO("Number of processed shared fragments");
	CHECK(Results.Num() == NumSharedFragments);
	for (int32 InitValue : TestInitValues)
	{
		INFO("Read values matches init values");
		CHECK(Results.Find(InitValue) != INDEX_NONE);
	}

	constexpr int32 ConditionalLimit = 10;
	TArray<int32> ConditionalResults;
	EntityManager->ForEachConstSharedFragmentConditional<FTestConstSharedFragment_Int>(
		[](const FTestConstSharedFragment_Int& SharedFragment)
		{
			return SharedFragment.Value > ConditionalLimit;
		}
		, [&ConditionalResults](const FTestConstSharedFragment_Int& SharedFragment)
		{
			ConditionalResults.Add(SharedFragment.Value);
		}
	);
	for (int32 Value : ConditionalResults)
	{
		INFO("Only the values matching the condition get processed");
		CHECK(Value > ConditionalLimit);
	}
}

//-----------------------------------------------------------------------------
// Entity-related Tests
//-----------------------------------------------------------------------------

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::SharedFragments::CreateEntities", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValueA = 1023;
	constexpr int32 TestIntValueB = 63;
	FMassEntityHandle EntityA, EntityB;

	CreateEntity<FSharedStruct, FTestSharedFragment_Int>(EntityA, TestIntValueA);
	CreateEntity<FSharedStruct, FTestSharedFragment_Int>(EntityB, TestIntValueB);

	INFO("Both entities should end up in the same archetype");
	CHECK(EntityManager->GetArchetypeForEntityUnsafe(EntityA) == EntityManager->GetArchetypeForEntityUnsafe(EntityB));

	FConstStructView SharedFragmentA = GetSharedFragmentView<FSharedStruct, FTestSharedFragment_Int>(EntityA);
	FConstStructView SharedFragmentB = GetSharedFragmentView<FSharedStruct, FTestSharedFragment_Int>(EntityB);

	INFO("SharedFragmentA should be valid");
	CHECK(SharedFragmentA.IsValid());
	INFO("SharedFragmentB should be valid");
	CHECK(SharedFragmentB.IsValid());
	INFO("SharedFragmentA should be of expected type");
	CHECK(SharedFragmentA.GetScriptStruct() == FTestSharedFragment_Int::StaticStruct());
	INFO("SharedFragmentB should be of expected type");
	CHECK(SharedFragmentB.GetScriptStruct() == FTestSharedFragment_Int::StaticStruct());
	INFO("SharedFragmentA and SharedFragmentB should be different instances");
	CHECK(SharedFragmentA != SharedFragmentB);
	INFO("SharedFragmentA's value should match the expected value");
	CHECK(SharedFragmentA.Get<const FTestSharedFragment_Int>().Value == TestIntValueA);
	INFO("SharedFragmentB's value should match the expected value");
	CHECK(SharedFragmentB.Get<const FTestSharedFragment_Int>().Value == TestIntValueB);
}

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::SharedFragments::CreateEntitiesConst", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValueA = 1023;
	constexpr int32 TestIntValueB = 63;
	FMassEntityHandle EntityA, EntityB;

	CreateEntity<FConstSharedStruct, FTestConstSharedFragment_Int>(EntityA, TestIntValueA);
	CreateEntity<FConstSharedStruct, FTestConstSharedFragment_Int>(EntityB, TestIntValueB);

	INFO("Both entities should end up in the same archetype");
	CHECK(EntityManager->GetArchetypeForEntityUnsafe(EntityA) == EntityManager->GetArchetypeForEntityUnsafe(EntityB));

	FConstStructView SharedFragmentA = GetSharedFragmentView<FConstSharedStruct, FTestConstSharedFragment_Int>(EntityA);
	FConstStructView SharedFragmentB = GetSharedFragmentView<FConstSharedStruct, FTestConstSharedFragment_Int>(EntityB);

	INFO("SharedFragmentA should be valid");
	CHECK(SharedFragmentA.IsValid());
	INFO("SharedFragmentB should be valid");
	CHECK(SharedFragmentB.IsValid());
	INFO("SharedFragmentA should be of expected type");
	CHECK(SharedFragmentA.GetScriptStruct() == FTestConstSharedFragment_Int::StaticStruct());
	INFO("SharedFragmentB should be of expected type");
	CHECK(SharedFragmentB.GetScriptStruct() == FTestConstSharedFragment_Int::StaticStruct());
	INFO("SharedFragmentA and SharedFragmentB should be different instances");
	CHECK(SharedFragmentA != SharedFragmentB);
	INFO("SharedFragmentA's value should match the expected value");
	CHECK(SharedFragmentA.Get<const FTestConstSharedFragment_Int>().Value == TestIntValueA);
	INFO("SharedFragmentB's value should match the expected value");
	CHECK(SharedFragmentB.Get<const FTestConstSharedFragment_Int>().Value == TestIntValueB);
}

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::SharedFragments::BatchCreateEntities", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValueA = 1023;
	constexpr int32 TestIntValueB = 63;
	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();
	// we create more than one chunk can handle to properly test moving entities between chunks
	const int32 EntitiesToCreateNumA = FMath::FloorToInt(static_cast<float>(EntitiesPerChunk) * 1.2f);
	const int32 EntitiesToCreateNumB = 1;
	constexpr int32 ExpectedNumberOfInitialChunks = 3;
	TArray<FMassEntityHandle> EntitiesA;
	TArray<FMassEntityHandle> EntitiesB;

	CreateEntities<FSharedStruct, FTestSharedFragment_Int>(EntitiesA, EntitiesToCreateNumA, TestIntValueA);
	CreateEntities<FSharedStruct, FTestSharedFragment_Int>(EntitiesB, EntitiesToCreateNumB, TestIntValueB);

	FMassArchetypeHandle CommonArchetype = EntityManager->GetArchetypeForEntityUnsafe(EntitiesA[0]);
	INFO("All the entities should end up in the same archetype");
	CHECK(FMassArchetypeHelper::ArchetypeDataFromHandleChecked(CommonArchetype).GetNumEntities() == EntitiesToCreateNumA + EntitiesToCreateNumB);
	INFO("The total number of chunks in the resulting archetype should match expectations");
	CHECK(FMassArchetypeHelper::ArchetypeDataFromHandleChecked(CommonArchetype).GetChunkCount() == ExpectedNumberOfInitialChunks);

	for (FMassEntityHandle EntityHandle : EntitiesA)
	{
		FConstStructView SharedFragment = GetSharedFragmentView<FSharedStruct, FTestSharedFragment_Int>(EntityHandle);
		INFO("SharedFragment for entity type A should be valid");
		CHECK(SharedFragment.IsValid());
		INFO("SharedFragment for entity type A should be of expected type");
		CHECK(SharedFragment.GetScriptStruct() == FTestSharedFragment_Int::StaticStruct());
		INFO("SharedFragment's value for entity type A should match the expected value");
		CHECK(SharedFragment.Get<const FTestSharedFragment_Int>().Value == TestIntValueA);
	}

	FConstStructView SharedFragment = GetSharedFragmentView<FSharedStruct, FTestSharedFragment_Int>(EntitiesB[0]);
	INFO("SharedFragment for entity type B should be valid");
	CHECK(SharedFragment.IsValid());
	INFO("SharedFragment for entity type B should be of expected type");
	CHECK(SharedFragment.GetScriptStruct() == FTestSharedFragment_Int::StaticStruct());
	INFO("SharedFragment's value for entity type B should match the expected value");
	CHECK(SharedFragment.Get<const FTestSharedFragment_Int>().Value == TestIntValueB);
}

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::SharedFragments::BatchCreateEntitiesConst", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValueA = 1023;
	constexpr int32 TestIntValueB = 63;
	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();
	const int32 EntitiesToCreateNumA = FMath::FloorToInt(static_cast<float>(EntitiesPerChunk) * 1.2f);
	const int32 EntitiesToCreateNumB = 1;
	constexpr int32 ExpectedNumberOfInitialChunks = 3;
	TArray<FMassEntityHandle> EntitiesA;
	TArray<FMassEntityHandle> EntitiesB;

	CreateEntities<FConstSharedStruct, FTestConstSharedFragment_Int>(EntitiesA, EntitiesToCreateNumA, TestIntValueA);
	CreateEntities<FConstSharedStruct, FTestConstSharedFragment_Int>(EntitiesB, EntitiesToCreateNumB, TestIntValueB);

	FMassArchetypeHandle CommonArchetype = EntityManager->GetArchetypeForEntityUnsafe(EntitiesA[0]);
	INFO("All the entities should end up in the same archetype");
	CHECK(FMassArchetypeHelper::ArchetypeDataFromHandleChecked(CommonArchetype).GetNumEntities() == EntitiesToCreateNumA + EntitiesToCreateNumB);
	INFO("The total number of chunks in the resulting archetype should match expectations");
	CHECK(FMassArchetypeHelper::ArchetypeDataFromHandleChecked(CommonArchetype).GetChunkCount() == ExpectedNumberOfInitialChunks);

	for (FMassEntityHandle EntityHandle : EntitiesA)
	{
		FConstStructView SharedFragment = GetSharedFragmentView<FConstSharedStruct, FTestConstSharedFragment_Int>(EntityHandle);
		INFO("SharedFragment for entity type A should be valid");
		CHECK(SharedFragment.IsValid());
		INFO("SharedFragment for entity type A should be of expected type");
		CHECK(SharedFragment.GetScriptStruct() == FTestConstSharedFragment_Int::StaticStruct());
		INFO("SharedFragment's value for entity type A should match the expected value");
		CHECK(SharedFragment.Get<const FTestConstSharedFragment_Int>().Value == TestIntValueA);
	}

	FConstStructView SharedFragment = GetSharedFragmentView<FConstSharedStruct, FTestConstSharedFragment_Int>(EntitiesB[0]);
	INFO("SharedFragment for entity type B should be valid");
	CHECK(SharedFragment.IsValid());
	INFO("SharedFragment for entity type B should be of expected type");
	CHECK(SharedFragment.GetScriptStruct() == FTestConstSharedFragment_Int::StaticStruct());
	INFO("SharedFragment's value for entity type B should match the expected value");
	CHECK(SharedFragment.Get<const FTestConstSharedFragment_Int>().Value == TestIntValueB);
}

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::ConstSharedFragments::AddToEntity", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 1023;
	const FTestConstSharedFragment_Int FragmentInstance(TestIntValue);

	const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

	const FTestConstSharedFragment_Int* EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
	INFO("Initially the entity is not expected to have the shared fragment");
	CHECK(EntitySharedFragment == nullptr);

	{
		const FConstSharedStruct& RegisteredSharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
		const bool bSuccessfullyAddedConstSharedFragment = EntityManager->AddConstSharedFragmentToEntity(EntityHandle, RegisteredSharedFragmentInstance);
		INFO("Adding registered const shared fragment struct should succeed");
		CHECK(bSuccessfullyAddedConstSharedFragment);
	}

	EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
	INFO("The entity is expected to have the shared fragment after the operation");
	REQUIRE(EntitySharedFragment != nullptr);
	INFO("The shared fragment is expected to store the configured value");
	CHECK(EntitySharedFragment->Value == TestIntValue);
	INFO("The entity's new archetype is not the same as the original one");
	CHECK(EntityManager->GetArchetypeForEntity(EntityHandle) != FloatsArchetype);
}

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::SharedFragments::AddToEntity", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 1023;
	const FTestSharedFragment_Int FragmentInstance(TestIntValue);
	const FSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateSharedFragment(FragmentInstance);

	const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

	const FTestSharedFragment_Int* EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
	INFO("Initially the entity is not expected to have the shared fragment");
	CHECK(EntitySharedFragment == nullptr);

	EntityManager->AddSharedFragmentToEntity(EntityHandle, SharedFragmentInstance);

	EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
	INFO("The entity is expected to have the shared fragment after the operation");
	REQUIRE(EntitySharedFragment != nullptr);
	INFO("The shared fragment is expected to store the configured value");
	CHECK(EntitySharedFragment->Value == TestIntValue);
}

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::ConstSharedFragments::RemoveFromEntity", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 1023;
	const FTestConstSharedFragment_Int FragmentInstance(TestIntValue);
	const FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);

	const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

	const FTestConstSharedFragment_Int* EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
	INFO("Initially the entity is not expected to have the shared fragment");
	CHECK(EntitySharedFragment == nullptr);

	INFO("Attempt to remove shared fragment from entity that doesn't have shared fragment should return false and do nothing");
	CHECK_FALSE(EntityManager->RemoveConstSharedFragmentFromEntity(EntityHandle, FTestConstSharedFragment_Int::StaticStruct()));

	INFO("Adding shared fragment to entity should succeed");
	CHECK(EntityManager->AddConstSharedFragmentToEntity(EntityHandle, SharedFragmentInstance));

	EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
	INFO("The entity is expected to have the shared fragment after the operation");
	REQUIRE(EntitySharedFragment != nullptr);
	INFO("The shared fragment is expected to store the configured value");
	CHECK(EntitySharedFragment->Value == TestIntValue);

	INFO("Removing shared fragment from entity that has the shared fragment should succeed");
	CHECK(EntityManager->RemoveConstSharedFragmentFromEntity(EntityHandle, FTestConstSharedFragment_Int::StaticStruct()));

	EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
	INFO("The entity is not expected to have the shared fragment after the operation");
	CHECK(EntitySharedFragment == nullptr);

	INFO("The entity's new archetype is the same as the initial one");
	CHECK(EntityManager->GetArchetypeForEntity(EntityHandle) == FloatsArchetype);
}

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::SharedFragments::RemoveFromEntity", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 1023;
	const FTestSharedFragment_Int FragmentInstance(TestIntValue);
	const FSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateSharedFragment(FragmentInstance);

	const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

	const FTestSharedFragment_Int* EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
	INFO("Initially the entity is not expected to have the shared fragment");
	CHECK(EntitySharedFragment == nullptr);

	INFO("Attempt to remove shared fragment from entity that doesn't have shared fragment should return false and do nothing");
	CHECK_FALSE(EntityManager->RemoveSharedFragmentFromEntity(EntityHandle, FTestSharedFragment_Int::StaticStruct()));

	INFO("Adding shared fragment to entity should succeed");
	CHECK(EntityManager->AddSharedFragmentToEntity(EntityHandle, SharedFragmentInstance));

	EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
	INFO("The entity is expected to have the shared fragment after the operation");
	REQUIRE(EntitySharedFragment != nullptr);
	INFO("The shared fragment is expected to store the configured value");
	CHECK(EntitySharedFragment->Value == TestIntValue);

	INFO("Removing shared fragment from entity that has the shared fragment should succeed");
	CHECK(EntityManager->RemoveSharedFragmentFromEntity(EntityHandle, FTestSharedFragment_Int::StaticStruct()));

	EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
	INFO("The entity is not expected to have the shared fragment after the operation");
	CHECK(EntitySharedFragment == nullptr);

	INFO("The entity's new archetype is the same as the initial one");
	CHECK(EntityManager->GetArchetypeForEntity(EntityHandle) == FloatsArchetype);
}

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::SharedFragments::BatchAddToEntity", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 1023;

	const FMassArchetypeHandle InitialArchetype = FloatsArchetype;
	const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(InitialArchetype).GetNumEntitiesPerChunk();
	const int32 EntitiesToCreateNum = FMath::FloorToInt(static_cast<float>(EntitiesPerChunk) * 2.2f);
	const int32 EntitiesToMoveNum = FMath::FloorToInt(static_cast<float>(EntitiesPerChunk) * 1.2f);

	TArray<FMassEntityHandle> CreatedEntityHandles;

	EntityManager->BatchCreateEntities(InitialArchetype, EntitiesToCreateNum, CreatedEntityHandles);

	TArray<FMassEntityHandle> EntitiesToMove = CreatedEntityHandles;
	Algo::RandomShuffle(EntitiesToMove);
	TConstArrayView<FMassEntityHandle> EntitiesMoved = MakeArrayView(EntitiesToMove.GetData(), EntitiesToMoveNum);
	FMassArchetypeEntityCollection EntityCollection(InitialArchetype, EntitiesMoved, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);

	FMassArchetypeSharedFragmentValues SharedValues;
	FConstSharedStruct ConstSharedFragment = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue);
	SharedValues.Add(ConstSharedFragment);
	EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&EntityCollection, 1), SharedValues);

	const FMassArchetypeHandle TargetArchetype = EntityManager->GetArchetypeForEntityUnsafe(EntitiesToMove[0]);
	const int32 EntitiesMovedNum = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(TargetArchetype).GetNumEntities();
	INFO("Number of entities moved needs to match expectations");
	CHECK(EntitiesMovedNum == EntitiesToMoveNum);
	for (const FMassEntityHandle& EntityHandle : EntitiesMoved)
	{
		FTestConstSharedFragment_Int* SharedFragmentInstance = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
		INFO("Every entity moved needs to have a valid shared fragment");
		REQUIRE(SharedFragmentInstance != nullptr);
		INFO("The shared fragment's value needs to match expectations");
		CHECK(SharedFragmentInstance->Value == TestIntValue);
	}
}

TEST_CASE_METHOD(FMassLLTSharedFragmentFixture, "Mass::SharedFragments::TypeEquivalency", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 TestIntValue = 32;

	FMassArchetypeSharedFragmentValues Values;
	const FMassSharedFragmentBitSet EmptySharedFragmentBitSet;
	const FMassConstSharedFragmentBitSet EmptyConstSharedFragmentBitSet;

	INFO("Empty shared values match type with empty bitset");
	CHECK(HasExactSharedFragmentTypesMatch(Values, EmptySharedFragmentBitSet));
	INFO("Empty const shared values match type with empty const bitset");
	CHECK(HasExactConstSharedFragmentTypesMatch(Values, EmptyConstSharedFragmentBitSet));

	const FMassSharedFragmentBitSet IntSharedFragmentBitSet = FMassSharedFragmentBitSet::GetTypeBitSet<FTestSharedFragment_Int>();
	FMassSharedFragmentBitSet IntFloatSharedFragmentBitSet = IntSharedFragmentBitSet;
	IntFloatSharedFragmentBitSet.Add<FTestSharedFragment_Float>();
	Values.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
	INFO("Single shared value type matches expected bitset");
	CHECK(HasExactSharedFragmentTypesMatch(Values, IntSharedFragmentBitSet));
	INFO("Single shared value type doesn't match two-type bitset");
	CHECK_FALSE(HasExactSharedFragmentTypesMatch(Values, IntFloatSharedFragmentBitSet));
	INFO("Single shared value type doesn't match empty");
	CHECK_FALSE(HasExactSharedFragmentTypesMatch(Values, EmptySharedFragmentBitSet));

	const FMassConstSharedFragmentBitSet IntConstSharedFragmentBitSet = FMassConstSharedFragmentBitSet::GetTypeBitSet<FTestConstSharedFragment_Int>();
	FMassConstSharedFragmentBitSet IntFloatConstSharedFragmentBitSet = IntConstSharedFragmentBitSet;
	IntFloatConstSharedFragmentBitSet.Add<FTestConstSharedFragment_Float>();
	Values.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
	INFO("Single const shared value type matches expected bitset");
	CHECK(HasExactConstSharedFragmentTypesMatch(Values, IntConstSharedFragmentBitSet));
	INFO("Single const shared value type doesn't match two-type bitset");
	CHECK_FALSE(HasExactConstSharedFragmentTypesMatch(Values, IntFloatConstSharedFragmentBitSet));
	INFO("Single const shared value type doesn't match empty");
	CHECK_FALSE(HasExactConstSharedFragmentTypesMatch(Values, EmptyConstSharedFragmentBitSet));

	Values.Remove(IntSharedFragmentBitSet);
	INFO("Emptied shared values match type with empty bitset");
	CHECK(HasExactSharedFragmentTypesMatch(Values, EmptySharedFragmentBitSet));
	Values.Remove(IntConstSharedFragmentBitSet);
	INFO("Emptied const shared values match type with empty const bitset");
	CHECK(HasExactConstSharedFragmentTypesMatch(Values, EmptyConstSharedFragmentBitSet));
}

//-----------------------------------------------------------------------------
// GetOrCreate Tests
//-----------------------------------------------------------------------------

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SharedFragments::GetOrCreate::WithArgs", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ConstIntValueOne = 1;
	constexpr int32 ConstIntValueTwo = 2;

	const FSharedStruct SharedFragment1 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>(/*Args*/ConstIntValueOne);
	const FSharedStruct SharedFragment2 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>(/*Args*/ConstIntValueOne);
	const FSharedStruct SharedFragment3 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>(/*Args*/ConstIntValueTwo);

	INFO("Shared fragments created for same struct type using same constructor value should share memory");
	CHECK(SharedFragment1 == SharedFragment2);
	INFO("Value in shared struct should be the same as the argument provided to GetOrCreateSharedFragment");
	CHECK(SharedFragment1.Get<FTestSharedFragment_Int>().Value == ConstIntValueOne);

	INFO("Shared fragments created for same struct type using different constructor values should not share memory");
	CHECK(SharedFragment1 != SharedFragment3);
	INFO("Value in shared struct should be the same as the argument provided to GetOrCreateSharedFragment");
	CHECK(SharedFragment3.Get<FTestSharedFragment_Int>().Value == ConstIntValueTwo);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SharedFragments::GetOrCreate::WithStruct", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ConstIntValueOne = 1;
	constexpr int32 ConstIntValueTwo = 2;

	const FTestSharedFragment_Int TestSharedFragment_Int1(ConstIntValueOne);
	const FTestSharedFragment_Int TestSharedFragment_Int2(ConstIntValueOne);
	const FTestSharedFragment_Int TestSharedFragment_Int3(ConstIntValueTwo);
	const FSharedStruct SharedFragment1 = EntityManager->GetOrCreateSharedFragment(TestSharedFragment_Int1);
	const FSharedStruct SharedFragment2 = EntityManager->GetOrCreateSharedFragment(TestSharedFragment_Int2);
	const FSharedStruct SharedFragment3 = EntityManager->GetOrCreateSharedFragment(TestSharedFragment_Int3);

	INFO("Shared fragments created for same struct type using same constructor value should share memory");
	CHECK(SharedFragment1 == SharedFragment2);
	INFO("Value in shared struct should be the same as the argument provided to GetOrCreateSharedFragment");
	CHECK(SharedFragment1.Get<FTestSharedFragment_Int>().Value == ConstIntValueOne);

	INFO("Shared fragments created for same struct type using different constructor values should not share memory");
	CHECK(SharedFragment1 != SharedFragment3);
	INFO("Value in shared struct should be the same as the argument provided to GetOrCreateSharedFragment");
	CHECK(SharedFragment3.Get<FTestSharedFragment_Int>().Value == ConstIntValueTwo);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SharedFragments::GetOrCreate::NoArgs", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	const FSharedStruct SharedFragment1 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>();
	const FSharedStruct SharedFragment2 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>();
	const FSharedStruct SharedFragment3 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>();

	INFO("Shared fragments created for same struct type using default constructor should share memory");
	CHECK(SharedFragment1 == SharedFragment2);
	INFO("Shared fragments created for same struct type using default constructor should share memory");
	CHECK(SharedFragment1 == SharedFragment3);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SharedFragments::GetOrCreate::WithArray", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	const FSharedStruct SharedFragment1 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Array>(TArray<int32>{ 1, 2, 3 });
	const FSharedStruct SharedFragment2 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Array>(TArray<int32>{ 1, 2, 3 });
	FTestSharedFragment_Array TestFragment;
	TestFragment.Value = { 1, 2, 3 };
	const FSharedStruct SharedFragment3 = EntityManager->GetOrCreateSharedFragment(TestFragment);

	INFO("Shared fragments created for same struct type using same TArray contents should share memory");
	CHECK(SharedFragment1 == SharedFragment2);
	INFO("Shared fragments created for same struct type using same TArray contents should share memory");
	CHECK(SharedFragment1 == SharedFragment3);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SharedFragments::GetOrCreate::ConstNoArgs", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	const FConstSharedStruct SharedFragment1 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>();
	const FConstSharedStruct SharedFragment2 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>();
	const FConstSharedStruct SharedFragment3 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>();

	INFO("Shared fragments created for same struct type using default constructor should share memory");
	CHECK(SharedFragment1 == SharedFragment2);
	INFO("Shared fragments created for same struct type using default constructor should share memory");
	CHECK(SharedFragment1 == SharedFragment3);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SharedFragments::GetOrCreate::ConstWithArgs", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ConstIntValueOne = 1;
	constexpr int32 ConstIntValueTwo = 2;

	const FConstSharedStruct SharedFragment1 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>(/*Args*/ConstIntValueOne);
	const FConstSharedStruct SharedFragment2 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>(/*Args*/ConstIntValueOne);
	const FConstSharedStruct SharedFragment3 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>(/*Args*/ConstIntValueTwo);

	INFO("Shared fragments created for same struct type using same constructor value should share memory");
	CHECK(SharedFragment1 == SharedFragment2);
	INFO("Value in shared struct should be the same as the argument provided to GetOrCreateConstSharedFragment");
	CHECK(SharedFragment1.Get<const FTestConstSharedFragment_Int>().Value == ConstIntValueOne);

	INFO("Shared fragments created for same struct type using different constructor values should not share memory");
	CHECK(SharedFragment1 != SharedFragment3);
	INFO("Value in shared struct should be the same as the argument provided to GetOrCreateConstSharedFragment");
	CHECK(SharedFragment3.Get<const FTestConstSharedFragment_Int>().Value == ConstIntValueTwo);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SharedFragments::GetOrCreate::ConstWithStruct", "[Mass][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ConstIntValueOne = 1;
	constexpr int32 ConstIntValueTwo = 2;

	const FTestConstSharedFragment_Int TestSharedFragment_Int1(ConstIntValueOne);
	const FTestConstSharedFragment_Int TestSharedFragment_Int2(ConstIntValueOne);
	const FTestConstSharedFragment_Int TestSharedFragment_Int3(ConstIntValueTwo);
	const FConstSharedStruct SharedFragment1 = EntityManager->GetOrCreateConstSharedFragment(TestSharedFragment_Int1);
	const FConstSharedStruct SharedFragment2 = EntityManager->GetOrCreateConstSharedFragment(TestSharedFragment_Int2);
	const FConstSharedStruct SharedFragment3 = EntityManager->GetOrCreateConstSharedFragment(TestSharedFragment_Int3);

	INFO("Shared fragments created for same struct type using same constructor value should share memory");
	CHECK(SharedFragment1 == SharedFragment2);
	INFO("Value in shared struct should be the same as the argument provided to GetOrCreateConstSharedFragment");
	CHECK(SharedFragment1.Get<const FTestConstSharedFragment_Int>().Value == ConstIntValueOne);

	INFO("Shared fragments created for same struct type using different constructor values should not share memory");
	CHECK(SharedFragment1 != SharedFragment3);
	INFO("Value in shared struct should be the same as the argument provided to GetOrCreateConstSharedFragment");
	CHECK(SharedFragment3.Get<const FTestConstSharedFragment_Int>().Value == ConstIntValueTwo);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
