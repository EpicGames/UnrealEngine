// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassArchetypeData.h"
#include "MassEntityBuilder.h"
#include "MassEntityManager.h"
#include "MassEntityMacros.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassSparseElementsStorage.h"

#include "TestHarness.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

#if WITH_MASSENTITY_DEBUG

//-----------------------------------------------------------------------------
// Helpers — ported from the AI test suite's helper structs
//-----------------------------------------------------------------------------
static TNotNull<UScriptStruct*> MakeElement(FName TypeName, TNotNull<UScriptStruct*> BaseElementType)
{
	UScriptStruct* NewScriptStruct = NewObject<UScriptStruct>(UScriptStruct::StaticClass(), FName(*FString::Printf(TEXT("%s"), *TypeName.ToString())), RF_Public);
	NewScriptStruct->SetSuperStruct(BaseElementType);
	NewScriptStruct->Bind();
	NewScriptStruct->PrepareCppStructOps();
	NewScriptStruct->StaticLink(true);
	NewScriptStruct->AddToRoot();
	return NewScriptStruct;
}

struct FAutoCleanupOperations
{
	~FAutoCleanupOperations()
	{
		for (UScriptStruct* Type : CreatedTypes)
		{
			Type->RemoveFromRoot();
		}
	}

	TArray<UScriptStruct*> CreatedTypes;
};

struct FTagTestOperations : FAutoCleanupOperations
{
	static bool HasSparseElement(const FMassArchetypeHandle& ArchetypeHandle, TNotNull<UScriptStruct*> SparseElementType)
	{
		return FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle).DoesContainEntitiesWithSparseElement(SparseElementType);
	}

	static bool HasSparseElement(FMassEntityView EntityView, TNotNull<UScriptStruct*> SparseElementType)
	{
		return EntityView.HasElement(SparseElementType);
	}

	TNotNull<UScriptStruct*> MakeSparseElement()
	{
		CreatedTypes.Add(MakeElement("SparseTagTestA", FMassSparseTag::StaticStruct()));
		return CreatedTypes.Last();
	}

	TNotNull<UScriptStruct*> MakeRegularElement()
	{
		CreatedTypes.Add(MakeElement("RegularTagTestA", FMassTag::StaticStruct()));
		return CreatedTypes.Last();
	}
};

struct FFragmentTestOperations : FAutoCleanupOperations
{
	static bool HasSparseElement(const FMassArchetypeHandle& ArchetypeHandle, TNotNull<UScriptStruct*> SparseElementType)
	{
		return FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle).DoesContainEntitiesWithSparseElement(SparseElementType);
	}

	static bool HasSparseElement(FMassEntityView EntityView, TNotNull<UScriptStruct*> SparseElementType)
	{
		return EntityView.HasElement(SparseElementType);
	}

	TNotNull<UScriptStruct*> MakeSparseElement()
	{
		CreatedTypes.Add(MakeElement("SparseFragmentTestA", FMassSparseFragment::StaticStruct()));
		return CreatedTypes.Last();
	}

	TNotNull<UScriptStruct*> MakeRegularElement()
	{
		CreatedTypes.Add(MakeElement("RegularFragmentTestA", FMassFragment::StaticStruct()));
		return CreatedTypes.Last();
	}
};

//-----------------------------------------------------------------------------
// Helper template to run a test body for both Tag and Fragment operations
//-----------------------------------------------------------------------------

//-----------------------------------------------------------------------------
// FSparseElements_Add — Tag
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Tag::Add", "[Mass][SparseElements]")
{
	FTagTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;
	constexpr int32 ModifiedEntityHandleIndex = 1;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	const FMassEntityHandle ModifiedEntityHandle = EntitiesCreated[ModifiedEntityHandleIndex];
	FStructView _ = EntityManager->AddSparseElementToEntity(ModifiedEntityHandle, SparseElementType);

	const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
	INFO("The affected entity is still in the original archetype");
	CHECK(NewArchetypeHandle == OriginalArchetypeHandle);
	INFO("The archetype has the element");
	CHECK(FTagTestOperations::HasSparseElement(NewArchetypeHandle, SparseElementType));

	FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
	INFO("The affected entity has the sparse element");
	CHECK(FTagTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));

	FMassEntityView OriginalEntityView(*EntityManager, EntitiesCreated[(ModifiedEntityHandleIndex + 1) % NumEntities]);
	INFO("(NOT) The original entity has the sparse element");
	CHECK_FALSE(FTagTestOperations::HasSparseElement(OriginalEntityView, SparseElementType));
}

//-----------------------------------------------------------------------------
// FSparseElements_Add — Fragment
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Fragment::Add", "[Mass][SparseElements]")
{
	FFragmentTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;
	constexpr int32 ModifiedEntityHandleIndex = 1;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	const FMassEntityHandle ModifiedEntityHandle = EntitiesCreated[ModifiedEntityHandleIndex];
	FStructView _ = EntityManager->AddSparseElementToEntity(ModifiedEntityHandle, SparseElementType);

	const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
	INFO("The affected entity is still in the original archetype");
	CHECK(NewArchetypeHandle == OriginalArchetypeHandle);
	INFO("The archetype has the element");
	CHECK(FFragmentTestOperations::HasSparseElement(NewArchetypeHandle, SparseElementType));

	FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
	INFO("The affected entity has the sparse element");
	CHECK(FFragmentTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));

	FMassEntityView OriginalEntityView(*EntityManager, EntitiesCreated[(ModifiedEntityHandleIndex + 1) % NumEntities]);
	INFO("(NOT) The original entity has the sparse element");
	CHECK_FALSE(FFragmentTestOperations::HasSparseElement(OriginalEntityView, SparseElementType));
}

//-----------------------------------------------------------------------------
// FSparseElements_Remove — Tag
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Tag::Remove", "[Mass][SparseElements]")
{
	FTagTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	for (FMassEntityHandle EntityHandle : EntitiesCreated)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(EntityHandle, SparseElementType);
	}

	EntityManager->RemoveSparseElementFromEntity(EntitiesCreated[0], SparseElementType);

	const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(EntitiesCreated[0]);
	INFO("The affected entity is still in the original archetype");
	CHECK(NewArchetypeHandle == OriginalArchetypeHandle);
	INFO("The archetype knows some of its entities have the sparse element");
	CHECK(FMassArchetypeHelper::ArchetypeDataFromHandleChecked(NewArchetypeHandle).ContainsAnySparseData());

	FMassEntityView ModifiedEntityView(*EntityManager, EntitiesCreated[0]);
	INFO("(NOT) The affected entity has the sparse element");
	CHECK_FALSE(FTagTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));

	FMassEntityView OtherEntityView(*EntityManager, EntitiesCreated[1]);
	INFO("The other entity has the sparse element");
	CHECK(FTagTestOperations::HasSparseElement(OtherEntityView, SparseElementType));

	EntityManager->RemoveSparseElementFromEntity(EntitiesCreated[1], SparseElementType);
	INFO("(NOT) The other entity has the sparse element");
	CHECK_FALSE(FTagTestOperations::HasSparseElement(OtherEntityView, SparseElementType));

	INFO("The original archetype no longer has the element");
	CHECK(FTagTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);
}

//-----------------------------------------------------------------------------
// FSparseElements_Remove — Fragment
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Fragment::Remove", "[Mass][SparseElements]")
{
	FFragmentTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	for (FMassEntityHandle EntityHandle : EntitiesCreated)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(EntityHandle, SparseElementType);
	}

	EntityManager->RemoveSparseElementFromEntity(EntitiesCreated[0], SparseElementType);

	const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(EntitiesCreated[0]);
	INFO("The affected entity is still in the original archetype");
	CHECK(NewArchetypeHandle == OriginalArchetypeHandle);
	INFO("The archetype knows some of its entities have the sparse element");
	CHECK(FMassArchetypeHelper::ArchetypeDataFromHandleChecked(NewArchetypeHandle).ContainsAnySparseData());

	FMassEntityView ModifiedEntityView(*EntityManager, EntitiesCreated[0]);
	INFO("(NOT) The affected entity has the sparse element");
	CHECK_FALSE(FFragmentTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));

	FMassEntityView OtherEntityView(*EntityManager, EntitiesCreated[1]);
	INFO("The other entity has the sparse element");
	CHECK(FFragmentTestOperations::HasSparseElement(OtherEntityView, SparseElementType));

	EntityManager->RemoveSparseElementFromEntity(EntitiesCreated[1], SparseElementType);
	INFO("(NOT) The other entity has the sparse element");
	CHECK_FALSE(FFragmentTestOperations::HasSparseElement(OtherEntityView, SparseElementType));

	INFO("The original archetype no longer has the element");
	CHECK(FFragmentTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);
}

//-----------------------------------------------------------------------------
// FSparseElements_MoveEntity — Tag
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Tag::Move", "[Mass][SparseElements]")
{
	FTagTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	const FMassEntityHandle ModifiedEntityHandle = EntityManager->CreateEntity(OriginalArchetypeHandle);

	FStructView _ = EntityManager->AddSparseElementToEntity(ModifiedEntityHandle, SparseElementType);

	EntityManager->AddFragmentToEntity(ModifiedEntityHandle, FTestFragment_Float::StaticStruct());

	INFO("The original archetype no longer has the element");
	CHECK(FTagTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

	const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
	INFO("The entity is expected to end up in the expected archetype");
	CHECK(NewArchetypeHandle == FloatsIntsArchetype);

	FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
	INFO("The affected entity has the sparse element");
	CHECK(FTagTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
}

//-----------------------------------------------------------------------------
// FSparseElements_MoveEntity — Fragment
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Fragment::Move", "[Mass][SparseElements]")
{
	FFragmentTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	const FMassEntityHandle ModifiedEntityHandle = EntityManager->CreateEntity(OriginalArchetypeHandle);

	FStructView _ = EntityManager->AddSparseElementToEntity(ModifiedEntityHandle, SparseElementType);

	EntityManager->AddFragmentToEntity(ModifiedEntityHandle, FTestFragment_Float::StaticStruct());

	INFO("The original archetype no longer has the element");
	CHECK(FFragmentTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

	const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
	INFO("The entity is expected to end up in the expected archetype");
	CHECK(NewArchetypeHandle == FloatsIntsArchetype);

	FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
	INFO("The affected entity has the sparse element");
	CHECK(FFragmentTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
}

//-----------------------------------------------------------------------------
// FSparseElements_BatchMoveEntities — Tag
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Tag::BatchMove", "[Mass][SparseElements]")
{
	FTagTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	for (FMassEntityHandle EntityHandle : EntitiesCreated)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(EntityHandle, SparseElementType);
	}

	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, EntitiesCreated, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchChangeFragmentCompositionForEntities({InitialCollection}, FMassFragmentBitSet(FTestFragment_Float::StaticStruct()), {});

	INFO("The original archetype no longer has the element");
	CHECK(FTagTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

	FMassEntityHandle ModifiedEntityHandle = EntitiesCreated[0];
	const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
	INFO("The entity is expected to end up in the expected archetype");
	CHECK(NewArchetypeHandle == FloatsIntsArchetype);

	FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
	INFO("The affected entity has the sparse element");
	CHECK(FTagTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
}

//-----------------------------------------------------------------------------
// FSparseElements_BatchMoveEntities — Fragment
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Fragment::BatchMove", "[Mass][SparseElements]")
{
	FFragmentTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	for (FMassEntityHandle EntityHandle : EntitiesCreated)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(EntityHandle, SparseElementType);
	}

	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, EntitiesCreated, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchChangeFragmentCompositionForEntities({InitialCollection}, FMassFragmentBitSet(FTestFragment_Float::StaticStruct()), {});

	INFO("The original archetype no longer has the element");
	CHECK(FFragmentTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

	FMassEntityHandle ModifiedEntityHandle = EntitiesCreated[0];
	const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
	INFO("The entity is expected to end up in the expected archetype");
	CHECK(NewArchetypeHandle == FloatsIntsArchetype);

	FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
	INFO("The affected entity has the sparse element");
	CHECK(FFragmentTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
}

//-----------------------------------------------------------------------------
// FSparseElements_BatchAdd — Tag
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Tag::BatchAdd", "[Mass][SparseElements]")
{
	FTagTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, EntitiesCreated, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({InitialCollection}, SparseElementType);

	INFO("The original archetype has the sparse element");
	CHECK(FTagTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType));

	for (FMassEntityHandle ModifiedEntityHandle : EntitiesCreated)
	{
		const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
		INFO("The entity is expected to end up in the expected archetype");
		CHECK(NewArchetypeHandle == OriginalArchetypeHandle);

		FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
		INFO("The affected entities have the sparse element");
		CHECK(FTagTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
	}
}

//-----------------------------------------------------------------------------
// FSparseElements_BatchAdd — Fragment
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Fragment::BatchAdd", "[Mass][SparseElements]")
{
	FFragmentTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, EntitiesCreated, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({InitialCollection}, SparseElementType);

	INFO("The original archetype has the sparse element");
	CHECK(FFragmentTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType));

	for (FMassEntityHandle ModifiedEntityHandle : EntitiesCreated)
	{
		const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
		INFO("The entity is expected to end up in the expected archetype");
		CHECK(NewArchetypeHandle == OriginalArchetypeHandle);

		FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
		INFO("The affected entities have the sparse element");
		CHECK(FFragmentTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
	}
}

//-----------------------------------------------------------------------------
// FSparseElements_BatchRemove — Tag
//   (inherits logic from BatchAdd, then removes)
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Tag::BatchRemove", "[Mass][SparseElements]")
{
	FTagTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, EntitiesCreated, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({InitialCollection}, SparseElementType);

	// Verify batch-add succeeded (same as BatchAdd test)
	INFO("The original archetype has the sparse element");
	CHECK(FTagTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType));

	for (FMassEntityHandle ModifiedEntityHandle : EntitiesCreated)
	{
		const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
		INFO("The entity is expected to end up in the expected archetype");
		CHECK(NewArchetypeHandle == OriginalArchetypeHandle);

		FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
		INFO("The affected entities have the sparse element");
		CHECK(FTagTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
	}

	// Now batch-remove
	EntityManager->BatchRemoveSparseElementFromEntities({InitialCollection}, SparseElementType);

	INFO("The original archetype no longer has the element");
	CHECK(FTagTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

	for (FMassEntityHandle ModifiedEntityHandle : EntitiesCreated)
	{
		const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
		INFO("The entity is expected to end up in the expected archetype");
		CHECK(NewArchetypeHandle == OriginalArchetypeHandle);

		FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
		INFO("(NOT) The affected entities have the sparse element");
		CHECK_FALSE(FTagTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
	}
}

//-----------------------------------------------------------------------------
// FSparseElements_BatchRemove — Fragment
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Fragment::BatchRemove", "[Mass][SparseElements]")
{
	FFragmentTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 2;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, EntitiesCreated, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({InitialCollection}, SparseElementType);

	// Verify batch-add succeeded (same as BatchAdd test)
	INFO("The original archetype has the sparse element");
	CHECK(FFragmentTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType));

	for (FMassEntityHandle ModifiedEntityHandle : EntitiesCreated)
	{
		const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
		INFO("The entity is expected to end up in the expected archetype");
		CHECK(NewArchetypeHandle == OriginalArchetypeHandle);

		FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
		INFO("The affected entities have the sparse element");
		CHECK(FFragmentTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
	}

	// Now batch-remove
	EntityManager->BatchRemoveSparseElementFromEntities({InitialCollection}, SparseElementType);

	INFO("The original archetype no longer has the element");
	CHECK(FFragmentTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

	for (FMassEntityHandle ModifiedEntityHandle : EntitiesCreated)
	{
		const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
		INFO("The entity is expected to end up in the expected archetype");
		CHECK(NewArchetypeHandle == OriginalArchetypeHandle);

		FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
		INFO("(NOT) The affected entities have the sparse element");
		CHECK_FALSE(FFragmentTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
	}
}

//-----------------------------------------------------------------------------
// FSparseElements_Query — Tag
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Query::Tag", "[Mass][SparseElements]")
{
	FTagTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 5;
	constexpr int32 NumEntitiesAffected = 3;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	TConstArrayView<FMassEntityHandle> ModifiedEntities = MakeArrayView(&EntitiesCreated[1], NumEntitiesAffected);
	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, ModifiedEntities, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({InitialCollection}, SparseElementType);

	// 1. Regular + Sparse
	{
		TArray<FMassEntityHandle> VerifyEntities1;

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddSparseRequirement(SparseElementType);

		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
		Query.ForEachEntityChunk(ExecutionContext, [SparseElementType, &VerifyEntities1](FMassExecutionContext& Context)
		{
			// option 1: the archetype has the sparse element, and we need to check individual entities
			for (auto EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				if (const bool bHasSparseElement = Context.HasSparseElement(SparseElementType, EntityIt))
				{
					// Tags don't have fragment data, so no view checks here
					VerifyEntities1.Add(Context.GetEntities()[EntityIt]);
				}
			}
			return true;
		});

		INFO("Number of expected entities, option 1");
		CHECK(VerifyEntities1.Num() == NumEntitiesAffected);

		TArray<FMassEntityHandle> VerifyEntities2;
		Query.ForEachEntityChunk(ExecutionContext, [SparseElementType, &VerifyEntities2](FMassExecutionContext& Context)
		{
			// option 2: dedicated sparse element iterator that skips entities that don't have the elements
			for (FMassSparseEntityIterator EntitySparseIt = Context.CreateSparseEntityIterator(); EntitySparseIt; ++EntitySparseIt)
			{
				const bool bHasSparseElement = Context.HasSparseElement(SparseElementType, EntitySparseIt);
				INFO("The filtered entity has the sparse element");
				CHECK(bHasSparseElement);

				VerifyEntities2.Add(Context.GetEntities()[EntitySparseIt]);
			}

			return true;
		});

		INFO("Number of expected entities, option 2");
		CHECK(VerifyEntities2.Num() == NumEntitiesAffected);
	}

	// 2. No-sparse
	{
		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddSparseRequirement(SparseElementType, EMassFragmentPresence::None);

		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);

		TArray<FMassEntityHandle> VerifyEntities;
		Query.ForEachEntityChunk(ExecutionContext, [SparseElementType, &VerifyEntities](FMassExecutionContext& Context)
		{
			for (FMassSparseEntityIterator EntitySparseIt = Context.CreateSparseEntityIterator(); EntitySparseIt; ++EntitySparseIt)
			{
				const bool bHasSparseElement = Context.HasSparseElement(SparseElementType, EntitySparseIt);
				INFO("(NOT) The filtered entity has the sparse element");
				CHECK_FALSE(bHasSparseElement);
				VerifyEntities.Add(Context.GetEntities()[EntitySparseIt]);
			}

			return true;
		});

		INFO("Number of expected entities, no-sparse");
		CHECK(VerifyEntities.Num() == (NumEntities - NumEntitiesAffected));
	}
}

//-----------------------------------------------------------------------------
// FSparseElements_Query — Fragment
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Query::Fragment", "[Mass][SparseElements]")
{
	FFragmentTestOperations TestOperations;
	UScriptStruct* SparseElementType = TestOperations.MakeSparseElement();
	FMassArchetypeHandle OriginalArchetypeHandle = IntsArchetype;

	constexpr int32 NumEntities = 5;
	constexpr int32 NumEntitiesAffected = 3;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

	TConstArrayView<FMassEntityHandle> ModifiedEntities = MakeArrayView(&EntitiesCreated[1], NumEntitiesAffected);
	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, ModifiedEntities, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({InitialCollection}, SparseElementType);

	// 1. Regular + Sparse
	{
		TArray<FMassEntityHandle> VerifyEntities1;

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddSparseRequirement(SparseElementType);

		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
		Query.ForEachEntityChunk(ExecutionContext, [SparseElementType, &VerifyEntities1](FMassExecutionContext& Context)
		{
			// option 1: the archetype has the sparse element, and we need to check individual entities
			for (auto EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				if (const bool bHasSparseElement = Context.HasSparseElement(SparseElementType, EntityIt))
				{
					if (UE::Mass::IsA<FMassFragment>(SparseElementType))
					{
						FConstStructView ElementView = Context.GetSparseElement(SparseElementType, EntityIt);
						INFO("The filtered entity has the sparse element, const view");
						CHECK(ElementView.IsValid());

						FStructView ElementMutableView = Context.GetMutableSparseElement(SparseElementType, EntityIt);
						INFO("The filtered entity has the sparse element, mutable view");
						CHECK(ElementMutableView.IsValid());

						INFO("Both views point at the same element");
						CHECK(ElementView == ElementMutableView);
					}

					VerifyEntities1.Add(Context.GetEntities()[EntityIt]);
				}
			}
			return true;
		});

		INFO("Number of expected entities, option 1");
		CHECK(VerifyEntities1.Num() == NumEntitiesAffected);

		TArray<FMassEntityHandle> VerifyEntities2;
		Query.ForEachEntityChunk(ExecutionContext, [SparseElementType, &VerifyEntities2](FMassExecutionContext& Context)
		{
			// option 2: dedicated sparse element iterator that skips entities that don't have the elements
			for (FMassSparseEntityIterator EntitySparseIt = Context.CreateSparseEntityIterator(); EntitySparseIt; ++EntitySparseIt)
			{
				const bool bHasSparseElement = Context.HasSparseElement(SparseElementType, EntitySparseIt);
				INFO("The filtered entity has the sparse element");
				CHECK(bHasSparseElement);

				if (UE::Mass::IsA<FMassFragment>(SparseElementType))
				{
					FConstStructView ElementView = Context.GetSparseElement(SparseElementType, EntitySparseIt);
					INFO("The filtered entity has the sparse element, const view");
					CHECK(ElementView.IsValid());

					FStructView ElementMutableView = Context.GetMutableSparseElement(SparseElementType, EntitySparseIt);
					INFO("The filtered entity has the sparse element, mutable view");
					CHECK(ElementMutableView.IsValid());

					INFO("Both views point at the same element");
					CHECK(ElementView == ElementMutableView);
				}

				VerifyEntities2.Add(Context.GetEntities()[EntitySparseIt]);
			}

			return true;
		});

		INFO("Number of expected entities, option 2");
		CHECK(VerifyEntities2.Num() == NumEntitiesAffected);
	}

	// 2. No-sparse
	{
		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddSparseRequirement(SparseElementType, EMassFragmentPresence::None);

		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);

		TArray<FMassEntityHandle> VerifyEntities;
		Query.ForEachEntityChunk(ExecutionContext, [SparseElementType, &VerifyEntities](FMassExecutionContext& Context)
		{
			for (FMassSparseEntityIterator EntitySparseIt = Context.CreateSparseEntityIterator(); EntitySparseIt; ++EntitySparseIt)
			{
				const bool bHasSparseElement = Context.HasSparseElement(SparseElementType, EntitySparseIt);
				INFO("(NOT) The filtered entity has the sparse element");
				CHECK_FALSE(bHasSparseElement);
				VerifyEntities.Add(Context.GetEntities()[EntitySparseIt]);
			}

			return true;
		});

		INFO("Number of expected entities, no-sparse");
		CHECK(VerifyEntities.Num() == (NumEntities - NumEntitiesAffected));
	}
}

//-----------------------------------------------------------------------------
// FSparseElements_QueryFragmentModification
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Query::ModifyFragment", "[Mass][SparseElements]")
{
	constexpr int32 NumEntities = 5;
	constexpr int32 NumEntitiesAffected = 3;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, EntitiesCreated);

	TConstArrayView<FMassEntityHandle> ModifiedEntities = MakeArrayView(&EntitiesCreated[1], NumEntitiesAffected);
	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, ModifiedEntities, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({InitialCollection}, FTestFragment_SparseInt::StaticStruct());

	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	Query.AddSparseRequirement<FTestFragment_SparseInt>();

	FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);

	int32 Counter = 0;
	TArray<FMassEntityHandle> VerifyEntities;
	Query.ForEachEntityChunk(ExecutionContext, [&Counter, &VerifyEntities](FMassExecutionContext& Context)
	{
		for (FMassSparseEntityIterator EntitySparseIt = Context.CreateSparseEntityIterator(); EntitySparseIt; ++EntitySparseIt)
		{
			FTestFragment_SparseInt* FragmentInstance = Context.GetMutableSparseElement<FTestFragment_SparseInt>(EntitySparseIt);
			INFO("The filtered entity has the sparse element");
			REQUIRE(FragmentInstance != nullptr);

			FragmentInstance->Value = Counter++;

			VerifyEntities.Add(Context.GetEntities()[EntitySparseIt]);
		}

		return true;
	});

	for (int32 EntityIndex = 0; EntityIndex < VerifyEntities.Num(); ++EntityIndex)
	{
		FMassEntityView EntityView(*EntityManager, VerifyEntities[EntityIndex]);
		FTestFragment_SparseInt* FragmentPtr = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		INFO("Collected entity's sparse fragment");
		REQUIRE(FragmentPtr != nullptr);
		INFO("Sparse fragment's expected value");
		CHECK(FragmentPtr->Value == EntityIndex);
	}
}

//-----------------------------------------------------------------------------
// FSparseElements_QuerySkipChunks
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Query::SkipChunks", "[Mass][SparseElements]")
{
	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
	constexpr int32 NumTotalChunks = 5;
	const int32 NumEntities = NumTotalChunks * EntitiesPerChunk;
	constexpr int32 NumAffectedChunks = 2;
	const int32 NumEntitiesAffected = NumAffectedChunks * EntitiesPerChunk;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, EntitiesCreated);

	// ModifiedEntities should contain all the entities in chunks 1 & 2
	TConstArrayView<FMassEntityHandle> ModifiedEntities = MakeArrayView(&EntitiesCreated[EntitiesPerChunk], NumEntitiesAffected);
	FMassArchetypeEntityCollection InitialCollection(IntsArchetype, ModifiedEntities, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({InitialCollection}, FTestFragment_SparseInt::StaticStruct());

	FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	Query.AddSparseRequirement<FTestFragment_SparseInt>();

	int32 Counter = 0;
	Query.ForEachEntityChunk(ExecutionContext, [&Counter](FMassExecutionContext& Context)
	{
		++Counter;
	});
	INFO("Expected number of chunks processed");
	CHECK(Counter == NumAffectedChunks);
}

//-----------------------------------------------------------------------------
// FSparseElements_Storage_Individual
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Storage::IndividualBasic", "[Mass][SparseElements]")
{
	FSparseElementsStorage Storage;

	const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(IntsArchetype);

	FConstStructView NoView = Storage.GetElementDataForEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
	INFO("Initially entity doesn't have the sparse element");
	CHECK(NoView.IsValid() == false);

	FStructView ElementView = Storage.AddElementToEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
	INFO("View of added element is valid");
	CHECK(ElementView.IsValid());

	FStructView ExistingView = Storage.GetMutableElementDataForEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
	INFO("Existing fragment view is valid");
	CHECK(ExistingView.IsValid());
	INFO("Both views point to the same fragment");
	CHECK(ElementView == ExistingView);

	const bool bRemoved = Storage.RemoveElementFromEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
	INFO("Removing the added element succeeds");
	CHECK(bRemoved);

	const bool bSecondRemove = Storage.RemoveElementFromEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
	INFO("Removing the added element for the second time fails");
	CHECK(bSecondRemove == false);

	FConstStructView RemovedView = Storage.GetElementDataForEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
	INFO("Fetching the removed element results in an invalid view");
	CHECK(RemovedView.IsValid() == false);
}

//-----------------------------------------------------------------------------
// FSparseElements_Storage_Iterator
//-----------------------------------------------------------------------------
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::SparseElements::Storage::Iterator", "[Mass][SparseElements]")
{
	constexpr int32 NumEntities = 100;

	FRandomStream RandStream(0);

	FSparseElementsStorage Storage;

	TArray<FMassEntityHandle> EntitiesCreated;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, EntitiesCreated);

	// add sparse elements
	TArray<FMassEntityHandle> ModifiedEntities;
	for (int32 EntityIndex = 0; EntityIndex < EntitiesCreated.Num(); ++EntityIndex)
	{
		if (RandStream.FRand() < 0.3f)
		{
			Storage.AddElementToEntity<FTestFragment_SparseInt>(EntitiesCreated[EntityIndex]);
			ModifiedEntities.Add(EntitiesCreated[EntityIndex]);
		}
	}

	int32 IteratedCount = 0;
	TSet<int32> IteratedEntityIndices;

	for (UE::Mass::FSparseElementIterator It = Storage.CreateElementIterator(FTestFragment_SparseInt::StaticStruct()); It; ++It)
	{
		++IteratedCount;

		FStructView ElementView = It.GetElementView();
		INFO("Iterated element view is valid");
		CHECK(ElementView.IsValid());

		const int32 EntityIndex = It.GetEntityIndex();
		IteratedEntityIndices.Add(EntityIndex);

		// perform modification, we'll test it later on
		FTestFragment_SparseInt* Fragment = ElementView.GetPtr<FTestFragment_SparseInt>();
		Fragment->Value = EntityIndex;
	}

	INFO("Iterated over all modified entities");
	CHECK(IteratedCount == ModifiedEntities.Num());

	for (const FMassEntityHandle& Entity : ModifiedEntities)
	{
		INFO("Iterated over modified entity");
		CHECK(IteratedEntityIndices.Contains(Entity.Index));
	}

	// test the values written
	for (UE::Mass::FSparseElementIterator It = Storage.CreateElementIterator(FTestFragment_SparseInt::StaticStruct()); It; ++It)
	{
		FConstStructView ElementView = It.GetElementView();
		INFO("Const view is valid");
		CHECK(ElementView.IsValid());

		const int32 EntityIndex = It.GetEntityIndex();
		const FTestFragment_SparseInt* Fragment = ElementView.GetPtr<FTestFragment_SparseInt>();
		INFO("Stored value matches expectations");
		CHECK(Fragment->Value == EntityIndex);
	}
}

#endif // WITH_MASSENTITY_DEBUG

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
