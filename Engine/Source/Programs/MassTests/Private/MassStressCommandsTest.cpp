// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "Algo/RandomShuffle.h"
#include "MassCommandBuffer.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

#if WITH_MASSENTITY_DEBUG

//----------------------------------------------------------------------//
// Sparse Element Command Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.AddSparseFragment", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

	// Add sparse fragments via command
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
	}

	// Verify sparse fragments not yet added
	for (int i = 0; i < Count; ++i)
	{
		INFO("Sparse fragment should not exist before flush");
		CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
	}

	EntityManager->FlushCommands();

	// Verify all entities have sparse fragments after flush
	for (int i = 0; i < Count; ++i)
	{
		INFO("Sparse fragment should exist after flush");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

		// Verify entities stayed in same archetype
		INFO("Entity should remain in original archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entities[i]) == IntsArchetype);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.AddSparseTag", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

	// Add sparse tags via command
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestTag_SparseA::StaticStruct()).Add(Entities[i]);
	}

	EntityManager->FlushCommands();

	// Verify all entities have sparse tags
	for (int i = 0; i < Count; ++i)
	{
		INFO("Sparse tag should exist after flush");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));

		// Verify entities stayed in same archetype
		INFO("Entity should remain in original archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entities[i]) == FloatsArchetype);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.RemoveSparseFragment", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

	// Add sparse fragments directly
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
	}

	// Verify all have sparse fragments
	for (int i = 0; i < Count; ++i)
	{
		INFO("Sparse fragment should exist");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
	}

	// Remove via command
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(
			FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
	}

	EntityManager->FlushCommands();

	// Verify all sparse fragments removed
	for (int i = 0; i < Count; ++i)
	{
		INFO("Sparse fragment should be removed after flush");
		CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

		// Verify entities stayed in same archetype
		INFO("Entity should remain in original archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entities[i]) == FloatsArchetype);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.RemoveSparseTag", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

	// Add sparse tags directly
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
	}

	// Remove via command
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(
			FTestTag_SparseA::StaticStruct()).Add(Entities[i]);
	}

	EntityManager->FlushCommands();

	// Verify all sparse tags removed
	for (int i = 0; i < Count; ++i)
	{
		INFO("Sparse tag should be removed after flush");
		CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.BatchAddSparseFragments", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
	const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f); // Span multiple chunks

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);

	// Batch add sparse fragments via single command with multiple entities
	EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
		FTestFragment_SparseInt::StaticStruct()).Add(Entities);

	EntityManager->FlushCommands();

	// Verify all entities have sparse fragments
	for (int i = 0; i < Count; ++i)
	{
		INFO("All entities should have sparse fragment after batch add");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

		// Verify entities stayed in same archetype
		INFO("Entity should remain in original archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entities[i]) == FloatsIntsArchetype);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.BatchRemoveSparseFragments", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
	const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

	// Add sparse fragments to all entities
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
	}

	// Batch remove via single command
	EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(
		FTestFragment_SparseFloat::StaticStruct()).Add(Entities);

	EntityManager->FlushCommands();

	// Verify all sparse fragments removed
	for (int i = 0; i < Count; ++i)
	{
		INFO("All sparse fragments should be removed after batch remove");
		CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.MixedSparseAndRegular", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

	// Mix regular fragment addition with sparse element addition
	for (int i = 0; i < Count; ++i)
	{
		// Add regular fragment (causes archetype change)
		EntityManager->Defer().AddFragment<FTestFragment_Float>(Entities[i]);
		// Add sparse fragment (no archetype change)
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
	}

	EntityManager->FlushCommands();

	// Verify entities moved to new archetype and have sparse elements
	for (int i = 0; i < Count; ++i)
	{
		INFO("Entity should be in FloatsInts archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entities[i]) == FloatsIntsArchetype);
		INFO("Entity should have sparse fragment");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

		// Verify regular fragment accessible
		INFO("Entity should have float fragment");
		REQUIRE(EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entities[i]) != nullptr);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.DataPreservation", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

	// Add sparse fragments and set values
	for (int i = 0; i < Count; ++i)
	{
		FTestFragment_SparseInt& SparseData = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
		SparseData.Value = i * 100;
	}

	// Trigger archetype change by adding regular fragment
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->Defer().AddFragment<FTestFragment_Int>(Entities[i]);
	}

	EntityManager->FlushCommands();

	// Verify sparse fragment data preserved across archetype change
	for (int i = 0; i < Count; ++i)
	{
		INFO("Sparse fragment should still exist");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

		FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
			FTestFragment_SparseInt::StaticStruct(), Entities[i]);
		INFO("Sparse fragment data should be preserved");
		CHECK(SparseData.Get<FTestFragment_SparseInt>().Value == i * 100);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.DuplicateAddHandling", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 5;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

	// Add same sparse fragment multiple times (should be idempotent)
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestFragment_SparseFloat::StaticStruct()).Add(Entities[i]);
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestFragment_SparseFloat::StaticStruct()).Add(Entities[i]);
	}

	EntityManager->FlushCommands();

	// Verify sparse fragment exists (should handle duplicates gracefully)
	for (int i = 0; i < Count; ++i)
	{
		INFO("Sparse fragment should exist after duplicate add");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.RemoveNonExistent", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 5;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

	// Try to remove sparse fragments that don't exist (should be safe)
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(
			FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
	}

	EntityManager->FlushCommands();

	// Verify entities still valid and in original archetype
	for (int i = 0; i < Count; ++i)
	{
		INFO("Entity should still be valid");
		CHECK(EntityManager->IsEntityValid(Entities[i]));
		INFO("Entity should remain in original archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entities[i]) == FloatsArchetype);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.MultipleTypes", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

	// Add multiple sparse element types to same entities
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestFragment_SparseFloat::StaticStruct()).Add(Entities[i]);
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestTag_SparseA::StaticStruct()).Add(Entities[i]);
	}

	EntityManager->FlushCommands();

	// Verify all sparse element types exist
	for (int i = 0; i < Count; ++i)
	{
		INFO("SparseInt should exist");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
		INFO("SparseFloat should exist");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
		INFO("SparseTag should exist");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));

		// All on same archetype
		INFO("Entity should remain in original archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entities[i]) == IntsArchetype);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.AcrossArchetypes", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 10;
	TArray<FMassEntityHandle> IntEntities;
	TArray<FMassEntityHandle> FloatEntities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, IntEntities);
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, FloatEntities);

	TArray<FMassEntityHandle> AllEntities;
	AllEntities.Append(IntEntities);
	AllEntities.Append(FloatEntities);

	// Add same sparse element to entities from different archetypes
	EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
		FTestFragment_SparseInt::StaticStruct()).Add(AllEntities);

	EntityManager->FlushCommands();

	// Verify all entities have the sparse element regardless of archetype
	for (const FMassEntityHandle& Entity : IntEntities)
	{
		INFO("IntArchetype entity should have sparse fragment");
		CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		INFO("Entity should remain in Ints archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);
	}

	for (const FMassEntityHandle& Entity : FloatEntities)
	{
		INFO("FloatArchetype entity should have sparse fragment");
		CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		INFO("Entity should remain in Floats archetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsArchetype);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.WithEntityDestruction", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

	// Add sparse fragments
	for (int i = 0; i < Count; ++i)
	{
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
		FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
			FTestFragment_SparseInt::StaticStruct(), Entities[i]);
		SparseData.Get<FTestFragment_SparseInt>().Value = i;
	}

	// Destroy half the entities via command
	TArray<FMassEntityHandle> EntitiesToDestroy;
	for (int i = 0; i < Count / 2; ++i)
	{
		EntitiesToDestroy.Add(Entities[i]);
		EntityManager->Defer().DestroyEntity(Entities[i]);
	}

	EntityManager->FlushCommands();

	// Verify destroyed entities are invalid
	for (int i = 0; i < Count / 2; ++i)
	{
		INFO("Destroyed entity should be invalid");
		CHECK_FALSE(EntityManager->IsEntityValid(Entities[i]));
	}

	// Verify remaining entities still have sparse fragments with correct data
	for (int i = Count / 2; i < Count; ++i)
	{
		INFO("Remaining entity should still be valid");
		CHECK(EntityManager->IsEntityValid(Entities[i]));
		INFO("Remaining entity should still have sparse fragment");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

		FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
			FTestFragment_SparseInt::StaticStruct(), Entities[i]);
		INFO("Sparse data should still be valid");
		CHECK(SparseData.Get<FTestFragment_SparseInt>().Value == i);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.SparseElements.PartialBatch", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 20;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

	// Add sparse fragment to first half only
	TArray<FMassEntityHandle> FirstHalf;
	for (int i = 0; i < Count / 2; ++i)
	{
		FirstHalf.Add(Entities[i]);
	}

	EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
		FTestFragment_SparseFloat::StaticStruct()).Add(FirstHalf);

	EntityManager->FlushCommands();

	// Verify first half has sparse fragments
	for (int i = 0; i < Count / 2; ++i)
	{
		INFO("First half should have sparse fragment");
		CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
	}

	// Verify second half does not have sparse fragments
	for (int i = Count / 2; i < Count; ++i)
	{
		INFO("Second half should not have sparse fragment");
		CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.BatchDestroy.WithSparseElements", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 20;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

	// Add sparse fragments and tags to all entities
	for (int i = 0; i < Count; ++i)
	{
		FTestFragment_SparseInt& SparseData = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
		SparseData.Value = i * 10;

		EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
	}

	// Verify sparse elements exist in storage
	const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();
	INFO("Storage should have all SparseInt elements");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == static_cast<uint32>(Count));

	// Verify each entity has sparse elements
	for (int i = 0; i < Count; ++i)
	{
		INFO("Entity should have SparseInt");
		CHECK(EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(Entities[i]));
	}

	// Destroy all entities using BatchDestroyEntities
	EntityManager->BatchDestroyEntities(Entities);

	// Verify all entities are invalid
	for (int i = 0; i < Count; ++i)
	{
		INFO("Entity should be invalid after destruction");
		CHECK_FALSE(EntityManager->IsEntityValid(Entities[i]));
	}

	// Verify sparse elements removed from storage
	INFO("Storage should have no SparseInt elements after destruction");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == 0u);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.BatchDestroy.PartialDestruction", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 Count = 30;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

	// Add sparse fragments to all entities with unique values
	for (int i = 0; i < Count; ++i)
	{
		FTestFragment_SparseFloat& SparseFloat = EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
		SparseFloat.Value = static_cast<float>(i);
	}

	const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();
	INFO("Storage should have all elements initially");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseFloat>() == static_cast<uint32>(Count));

	// Destroy first third of entities
	TArray<FMassEntityHandle> EntitiesToDestroy;
	for (int i = 0; i < Count / 3; ++i)
	{
		EntitiesToDestroy.Add(Entities[i]);
	}

	EntityManager->BatchDestroyEntities(EntitiesToDestroy);

	// Verify destroyed entities are invalid
	for (int i = 0; i < Count / 3; ++i)
	{
		INFO("Destroyed entity should be invalid");
		CHECK_FALSE(EntityManager->IsEntityValid(Entities[i]));
	}

	// Verify remaining entities still valid with correct data
	const int32 ExpectedRemaining = Count - Count / 3;
	INFO("Storage should have correct number of remaining elements");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseFloat>() == static_cast<uint32>(ExpectedRemaining));

	for (int i = Count / 3; i < Count; ++i)
	{
		INFO("Remaining entity should still be valid");
		CHECK(EntityManager->IsEntityValid(Entities[i]));
		INFO("Remaining entity should still have sparse element");
		CHECK(EntityManager->DoesEntityHaveElement<FTestFragment_SparseFloat>(Entities[i]));

		FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
			FTestFragment_SparseFloat::StaticStruct(), Entities[i]);
		INFO("Sparse data should be preserved for remaining entities");
		CHECK(SparseData.Get<FTestFragment_SparseFloat>().Value == static_cast<float>(i));
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.BatchDestroyChunks.WithSparseElements", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
	const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f); // Span multiple chunks

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);

	// Add multiple sparse element types to all entities
	for (int i = 0; i < Count; ++i)
	{
		FTestFragment_SparseInt& SparseInt = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
		SparseInt.Value = i;

		FTestFragment_SparseFloat& SparseFloat = EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
		SparseFloat.Value = static_cast<float>(i * 2);

		EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
	}

	const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();
	INFO("Storage should have all SparseInt elements");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == static_cast<uint32>(Count));
	INFO("Storage should have all SparseFloat elements");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseFloat>() == static_cast<uint32>(Count));

	// Create entity collection for batch chunk destruction
	TArray<FMassArchetypeEntityCollection> EntityCollections;
	UE::Mass::Utils::CreateEntityCollections(*EntityManager, Entities,
		FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates, EntityCollections);

	// Destroy all entity chunks
	EntityManager->BatchDestroyEntityChunks(EntityCollections);

	// Verify all entities are invalid
	for (int i = 0; i < Count; ++i)
	{
		INFO("Entity should be invalid after chunk destruction");
		CHECK_FALSE(EntityManager->IsEntityValid(Entities[i]));
	}

	// Verify all sparse elements removed from storage
	INFO("Storage should have no SparseInt elements after chunk destruction");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == 0u);
	INFO("Storage should have no SparseFloat elements after chunk destruction");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseFloat>() == 0u);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.BatchDestroyChunks.PartialChunks", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
	const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 3.0f); // 3 full chunks

	TArray<FMassEntityHandle> AllEntities;
	EntityManager->BatchCreateEntities(IntsArchetype, Count, AllEntities);

	// Add sparse elements to all entities
	for (int i = 0; i < Count; ++i)
	{
		FTestFragment_SparseInt& SparseInt = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(AllEntities[i]);
		SparseInt.Value = i * 100;
	}

	const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();
	INFO("Storage should have all elements initially");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == static_cast<uint32>(Count));

	// Select only first 2 chunks worth of entities to destroy
	const int32 EntitiesToDestroy = EntitiesPerChunk * 2;
	TArray<FMassEntityHandle> EntitiesToDestroyArray;
	EntitiesToDestroyArray.Reserve(EntitiesToDestroy);
	for (int i = 0; i < EntitiesToDestroy; ++i)
	{
		EntitiesToDestroyArray.Add(AllEntities[i]);
	}

	// Create entity collection and destroy chunks
	TArray<FMassArchetypeEntityCollection> EntityCollections;
	UE::Mass::Utils::CreateEntityCollections(*EntityManager, EntitiesToDestroyArray,
		FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates, EntityCollections);

	EntityManager->BatchDestroyEntityChunks(EntityCollections);

	// Verify destroyed entities are invalid
	for (int i = 0; i < EntitiesToDestroy; ++i)
	{
		INFO("Destroyed entity should be invalid");
		CHECK_FALSE(EntityManager->IsEntityValid(AllEntities[i]));
	}

	// Verify remaining entities (third chunk) still valid with correct data
	const int32 ExpectedRemaining = Count - EntitiesToDestroy;
	INFO("Storage should have correct number of remaining elements");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == static_cast<uint32>(ExpectedRemaining));

	for (int i = EntitiesToDestroy; i < Count; ++i)
	{
		INFO("Remaining entity should still be valid");
		CHECK(EntityManager->IsEntityValid(AllEntities[i]));
		INFO("Remaining entity should still have sparse element");
		CHECK(EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(AllEntities[i]));

		FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
			FTestFragment_SparseInt::StaticStruct(), AllEntities[i]);
		INFO("Sparse data should be preserved for remaining entities");
		CHECK(SparseData.Get<FTestFragment_SparseInt>().Value == i * 100);
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Commands.BatchDestroy.MixedSparseTypes", "[Mass][Stress][Commands][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 Count = 25;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);

	// Add different combinations of sparse elements to different entities
	// First third: SparseInt + SparseFloat + SparseTag
	for (int i = 0; i < Count / 3; ++i)
	{
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
		EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
	}

	// Middle third: SparseInt + SparseTag only
	for (int i = Count / 3; i < (Count * 2) / 3; ++i)
	{
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
		EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
	}

	// Last third: SparseFloat only
	for (int i = (Count * 2) / 3; i < Count; ++i)
	{
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
	}

	const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();

	// Verify initial counts
	constexpr uint32 InitialSparseIntCount = (Count / 3) + ((Count * 2) / 3 - Count / 3);
	constexpr uint32 InitialSparseFloatCount = (Count / 3) + (Count - (Count * 2) / 3);

	INFO("Initial SparseInt count should be correct");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == InitialSparseIntCount);
	INFO("Initial SparseFloat count should be correct");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseFloat>() == InitialSparseFloatCount);

	// Destroy all entities
	EntityManager->BatchDestroyEntities(Entities);

	// Verify all sparse elements removed regardless of type combinations
	INFO("All SparseInt elements should be removed");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == 0u);
	INFO("All SparseFloat elements should be removed");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseFloat>() == 0u);
}


#endif // WITH_MASSENTITY_DEBUG

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
