// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityBuilder.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

//-----------------------------------------------------------------------------
// Sparse Elements Tests
//-----------------------------------------------------------------------------

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.AddSparseFrag_Templated", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create entity with sparse fragment using templated Add<>
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FMassEntityHandle Entity = Builder.Add<FTestFragment_SparseInt>(100)
		.Commit();

	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify sparse element was added to storage
	FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("Sparse element should exist in storage");
	CHECK(SparseView.IsValid());

	const FTestFragment_SparseInt* SparseData = SparseView.GetPtr<FTestFragment_SparseInt>();
	INFO("Sparse data should be accessible");
	REQUIRE(SparseData != nullptr);
	INFO("Sparse value should match initialization");
	CHECK(SparseData->Value == 100);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.AddSparseTag_Templated", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create entity with sparse tag using templated Add<>
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FMassEntityHandle Entity = Builder.Add<FTestTag_SparseA>()
		.Commit();

	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify sparse tag was added
	const bool bHasTag = EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct());

	INFO("Entity should have sparse tag");
	CHECK(bHasTag);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.AddSparse_Mixed", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create entity with mix of regular and sparse fragments
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FMassEntityHandle Entity = Builder.Add<FTestFragment_Int>(42)
		.Add<FTestFragment_SparseInt>(100)
		.Add<FTestFragment_Float>(3.14f)
		.Add<FTestFragment_SparseFloat>(2.718f)
		.Add<FTestTag_A>()
		.Add<FTestTag_SparseB>()
		.Commit();

	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify regular fragments
	const FTestFragment_Int& RegularInt = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity);
	INFO("Regular int fragment should match");
	CHECK(RegularInt.Value == 42);

	const FTestFragment_Float& RegularFloat = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
	INFO("Regular float fragment should match");
	CHECK(RegularFloat.Value == 3.14f);

	// Verify regular tag
	INFO("Entity should have regular tag");
	CHECK(EntityManager->DoesEntityHaveElement<FTestTag_A>(Entity));

	// Verify sparse fragments
	FConstStructView SparseIntView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("Sparse int should exist");
	CHECK(SparseIntView.IsValid());
	INFO("Sparse int value should match");
	CHECK(SparseIntView.GetPtr<FTestFragment_SparseInt>()->Value == 100);

	FConstStructView SparseFloatView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseFloat::StaticStruct(), Entity);
	INFO("Sparse float should exist");
	CHECK(SparseFloatView.IsValid());
	INFO("Sparse float value should match");
	CHECK(SparseFloatView.GetPtr<FTestFragment_SparseFloat>()->Value == 2.718f);

	// Verify sparse tag
	INFO("Entity should have sparse tag");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.AddSparse_InstancedStruct", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create sparse fragment instance
	FInstancedStruct SparseInstance = FInstancedStruct::Make<FTestFragment_SparseInt>(200);

	// Create entity using Add(FInstancedStruct)
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FMassEntityHandle Entity = Builder.Add(SparseInstance)
		.Commit();

	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify sparse element
	FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("Sparse element should exist");
	CHECK(SparseView.IsValid());
	INFO("Sparse value should match");
	CHECK(SparseView.GetPtr<FTestFragment_SparseInt>()->Value == 200);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.AddSparse_UScriptStruct", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Add sparse fragment by type pointer (no data initialization)
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FMassEntityHandle Entity = Builder.Add(FTestFragment_SparseInt::StaticStruct())
		.Commit();

	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify sparse element exists with default value
	FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("Sparse element should exist");
	CHECK(SparseView.IsValid());
	INFO("Sparse value should be default initialized (0)");
	CHECK(SparseView.GetPtr<FTestFragment_SparseInt>()->Value == 0);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.AddGetRef_Sparse", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Use Add_GetRef to get mutable reference and modify
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FTestFragment_SparseInt& SparseRef = Builder.Add_GetRef<FTestFragment_SparseInt>();
	SparseRef.Value = 500;

	FMassEntityHandle Entity = Builder.Commit();
	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify value was set through reference
	FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("Sparse element should exist");
	CHECK(SparseView.IsValid());
	INFO("Sparse value should match reference modification");
	CHECK(SparseView.GetPtr<FTestFragment_SparseInt>()->Value == 500);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.GetOrCreate_Sparse", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Use GetOrCreate twice - should return same instance
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FTestFragment_SparseInt& FirstRef = Builder.GetOrCreate<FTestFragment_SparseInt>(100);
	FirstRef.Value = 200;  // Modify

	FTestFragment_SparseInt& SecondRef = Builder.GetOrCreate<FTestFragment_SparseInt>(999);  // Should override
	INFO("GetOrCreate should override existing value");
	CHECK(SecondRef.Value == 999);

	FMassEntityHandle Entity = Builder.Commit();
	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify final value
	FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("Sparse element should exist");
	CHECK(SparseView.IsValid());
	INFO("Final sparse value should be from second GetOrCreate");
	CHECK(SparseView.GetPtr<FTestFragment_SparseInt>()->Value == 999);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.Find_Sparse", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create builder with sparse fragment
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	Builder.Add<FTestFragment_SparseInt>(300);

	// Find should return non-null pointer
	FTestFragment_SparseInt* Found = Builder.Find<FTestFragment_SparseInt>();
	INFO("Find should return non-null for added sparse fragment");
	REQUIRE(Found != nullptr);
	INFO("Found value should match");
	CHECK(Found->Value == 300);

	// Find for non-added type should return null
	FTestFragment_SparseFloat* NotFound = Builder.Find<FTestFragment_SparseFloat>();
	INFO("Find should return null for non-added sparse fragment");
	CHECK(NotFound == nullptr);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.MultipleSparseFragments", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create entity with multiple sparse fragments
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FMassEntityHandle Entity = Builder.Add<FTestFragment_SparseInt>(111)
		.Add<FTestFragment_SparseFloat>(222.0f)
		.Commit();

	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify both sparse fragments
	FConstStructView SparseIntView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("First sparse element should exist");
	CHECK(SparseIntView.IsValid());
	INFO("First sparse value should match");
	CHECK(SparseIntView.GetPtr<FTestFragment_SparseInt>()->Value == 111);

	FConstStructView SparseFloatView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseFloat::StaticStruct(), Entity);
	INFO("Second sparse element should exist");
	CHECK(SparseFloatView.IsValid());
	INFO("Second sparse value should match");
	CHECK(SparseFloatView.GetPtr<FTestFragment_SparseFloat>()->Value == 222.0f);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.MultipleSparseTags", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create entity with multiple sparse tags
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FMassEntityHandle Entity = Builder.Add<FTestTag_SparseA>()
		.Add<FTestTag_SparseB>()
		.Commit();

	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify both sparse tags
	INFO("Entity should have first sparse tag");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
	INFO("Entity should have second sparse tag");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.CopyDataFromEntity_WithSparse", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create source entity with sparse elements
	UE::Mass::FEntityBuilder SourceBuilder(*EntityManager);
	FMassEntityHandle SourceEntity = SourceBuilder.Add<FTestFragment_Int>(42)
		.Add<FTestFragment_SparseInt>(999)
		.Add<FTestTag_SparseA>()
		.Commit();

	// Create new entity by copying from source
	UE::Mass::FEntityBuilder CopyBuilder(*EntityManager);
	const bool bSuccess = CopyBuilder.CopyDataFromEntity(SourceEntity);
	INFO("CopyDataFromEntity should succeed");
	CHECK(bSuccess);

	FMassEntityHandle CopiedEntity = CopyBuilder.Commit();
	INFO("Copied entity should be valid");
	CHECK(CopiedEntity.IsValid());

	// Verify regular fragment was copied
	const FTestFragment_Int& CopiedRegular = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CopiedEntity);
	INFO("Regular fragment should be copied");
	CHECK(CopiedRegular.Value == 42);

	// Verify sparse fragment was copied
	FConstStructView CopiedSparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), CopiedEntity);
	INFO("Sparse fragment should be copied");
	CHECK(CopiedSparseView.IsValid());
	INFO("Sparse fragment value should match");
	CHECK(CopiedSparseView.GetPtr<FTestFragment_SparseInt>()->Value == 999);

	// Verify sparse tag was copied
	INFO("Sparse tag should be copied");
	CHECK(EntityManager->DoesEntityHaveElement(CopiedEntity, FTestTag_SparseA::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.AppendDataFromEntity_WithSparse", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create source entity with sparse elements
	UE::Mass::FEntityBuilder SourceBuilder(*EntityManager);
	FMassEntityHandle SourceEntity = SourceBuilder.Add<FTestFragment_SparseInt>(777)
		.Add<FTestTag_SparseA>()
		.Commit();

	// Create builder with some existing data
	UE::Mass::FEntityBuilder AppendBuilder(*EntityManager);
	AppendBuilder.Add<FTestFragment_Float>(1.5f)
		.Add<FTestTag_SparseB>();

	// Append data from source
	const bool bSuccess = AppendBuilder.AppendDataFromEntity(SourceEntity);
	INFO("AppendDataFromEntity should succeed");
	CHECK(bSuccess);

	FMassEntityHandle ResultEntity = AppendBuilder.Commit();
	INFO("Result entity should be valid");
	CHECK(ResultEntity.IsValid());

	// Verify original data is still there
	const FTestFragment_Float& OriginalFloat = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(ResultEntity);
	INFO("Original fragment should remain");
	CHECK(OriginalFloat.Value == 1.5f);
	INFO("Original sparse tag should remain");
	CHECK(EntityManager->DoesEntityHaveElement(ResultEntity, FTestTag_SparseB::StaticStruct()));

	// Verify appended sparse data
	FConstStructView AppendedSparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), ResultEntity);
	INFO("Appended sparse fragment should exist");
	CHECK(AppendedSparseView.IsValid());
	INFO("Appended sparse value should match");
	CHECK(AppendedSparseView.GetPtr<FTestFragment_SparseInt>()->Value == 777);

	INFO("Appended sparse tag should exist");
	CHECK(EntityManager->DoesEntityHaveElement(ResultEntity, FTestTag_SparseA::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.Make_WithSparseFragments", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Prepare composition and initial values
	FMassArchetypeCompositionDescriptor Composition;
	Composition.Add<FTestFragment_Int>();
	Composition.Add<FTestFragment_SparseInt>();
	Composition.Add<FTestTag_SparseA>();

	TArray<FInstancedStruct> InitialFragments;
	InitialFragments.Add(FInstancedStruct::Make<FTestFragment_Int>(50));
	InitialFragments.Add(FInstancedStruct::Make<FTestFragment_SparseInt>(888));

	// Create builder using Make with sparse fragments
	UE::Mass::FEntityBuilder Builder = UE::Mass::FEntityBuilder::Make(
		EntityManager->AsShared(),
		Composition,
		InitialFragments
	);

	FMassEntityHandle Entity = Builder.Commit();
	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify regular fragment
	const FTestFragment_Int& RegularInt = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity);
	INFO("Regular fragment should match");
	CHECK(RegularInt.Value == 50);

	// Verify sparse fragment
	FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("Sparse fragment should exist");
	CHECK(SparseView.IsValid());
	INFO("Sparse fragment value should match");
	CHECK(SparseView.GetPtr<FTestFragment_SparseInt>()->Value == 888);

	// Verify sparse tag
	INFO("Sparse tag should exist");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.CommitAndReprepare_WithSparse", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create builder with sparse elements
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	Builder.Add<FTestFragment_SparseInt>(111);

	// Commit first entity
	FMassEntityHandle Entity1 = Builder.CommitAndReprepare();
	INFO("First entity should be valid");
	CHECK(Entity1.IsValid());

	// Modify and commit second entity
	FTestFragment_SparseInt& Ref = Builder.GetOrCreate<FTestFragment_SparseInt>(222);
	INFO("Second entity data should be modified");
	CHECK(Ref.Value == 222);

	FMassEntityHandle Entity2 = Builder.Commit();
	INFO("Second entity should be valid");
	CHECK(Entity2.IsValid());
	INFO("Entities should be different");
	CHECK(Entity1.Index != Entity2.Index);

	// Verify both entities have correct values
	FConstStructView Sparse1 = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity1);
	INFO("The Sparse1 fragment view is expected to be valid");
	CHECK(Sparse1.IsValid());
	INFO("First entity should have original value");
	CHECK(Sparse1.GetPtr<FTestFragment_SparseInt>()->Value == 111);

	FConstStructView Sparse2 = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity2);
	INFO("The Sparse2 fragment view is expected to be valid");
	CHECK(Sparse2.IsValid());
	INFO("Second entity should have modified value");
	CHECK(Sparse2.GetPtr<FTestFragment_SparseInt>()->Value == 222);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.DeferredCommit_WithSparse", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create builder with deferred commit enabled
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	Builder.SetForceDeferredCommit(true);
	Builder.Add<FTestFragment_SparseInt>(333)
		.Add<FTestTag_SparseA>();

	FMassEntityHandle Entity = Builder.Commit();
	INFO("Entity should be valid after deferred commit");
	CHECK(Entity.IsValid());

	// Flush commands
	EntityManager->FlushCommands();

	// Verify sparse elements were added through deferred path
	FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("Sparse fragment should exist after flush");
	CHECK(SparseView.IsValid());
	INFO("Sparse value should match");
	CHECK(SparseView.GetPtr<FTestFragment_SparseInt>()->Value == 333);

	INFO("Sparse tag should exist after flush");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityBuilder.SparseElements.SparseOnly_NoRegularFragments", "[Mass][Stress][EntityBuilder]")
{
	REQUIRE(EntityManager);

	// Create entity with ONLY sparse elements (no regular fragments)
	UE::Mass::FEntityBuilder Builder(*EntityManager);
	FMassEntityHandle Entity = Builder.Add<FTestFragment_SparseInt>(444)
		.Add<FTestFragment_SparseFloat>(555.0f)
		.Add<FTestTag_SparseA>()
		.Add<FTestTag_SparseB>()
		.Commit();

	INFO("Entity should be valid");
	CHECK(Entity.IsValid());

	// Verify all sparse elements exist
	FConstStructView SparseIntView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
	INFO("Sparse int should exist");
	CHECK(SparseIntView.IsValid());
	INFO("Sparse int value should match");
	CHECK(SparseIntView.GetPtr<FTestFragment_SparseInt>()->Value == 444);

	FConstStructView SparseFloatView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseFloat::StaticStruct(), Entity);
	INFO("Sparse float should exist");
	CHECK(SparseFloatView.IsValid());
	INFO("Sparse float value should match");
	CHECK(SparseFloatView.GetPtr<FTestFragment_SparseFloat>()->Value == 555.0f);

	INFO("First sparse tag should exist");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
	INFO("Second sparse tag should exist");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
