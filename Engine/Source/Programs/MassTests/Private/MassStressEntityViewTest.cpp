// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityManager.h"
#include "MassEntityView.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

#if WITH_MASSENTITY_DEBUG

//----------------------------------------------------------------------//
// FMassEntityView Sparse Element Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.HasFragment.PerEntity", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	// Test bug: HasFragment should check per-entity sparse element, not just archetype
	const FMassEntityHandle Entity1 = EntityManager->CreateEntity(IntsArchetype);
	const FMassEntityHandle Entity2 = EntityManager->CreateEntity(IntsArchetype);

	// Add sparse fragment to only Entity1
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity1);

	FMassEntityView View1(*EntityManager, Entity1);
	FMassEntityView View2(*EntityManager, Entity2);

	// Both are in same archetype, but only Entity1 has the sparse fragment
	INFO("Both entities in same archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity1) == EntityManager->GetArchetypeForEntity(Entity2));

	INFO("Entity1 should have sparse fragment");
	CHECK(View1.HasElement<FTestFragment_SparseInt>());
	INFO("Entity2 should NOT have sparse fragment");
	CHECK_FALSE(View2.HasElement<FTestFragment_SparseInt>());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.HasTag.PerEntity", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity1 = EntityManager->CreateEntity(FloatsArchetype);
	const FMassEntityHandle Entity2 = EntityManager->CreateEntity(FloatsArchetype);

	// Add sparse tag to only Entity1
	EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entity1);

	FMassEntityView View1(*EntityManager, Entity1);
	FMassEntityView View2(*EntityManager, Entity2);

	INFO("Entity1 should have sparse tag");
	CHECK(View1.HasElement<FTestTag_SparseA>());
	INFO("Entity2 should NOT have sparse tag");
	CHECK_FALSE(View2.HasElement<FTestTag_SparseA>());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.HasElement.BugTest", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	// Regression: HasElement must check per-entity sparse elements, not just archetype composition
	const FMassEntityHandle Entity1 = EntityManager->CreateEntity(IntsArchetype);
	const FMassEntityHandle Entity2 = EntityManager->CreateEntity(IntsArchetype);

	// Add sparse fragment to only Entity1
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity1);

	FMassEntityView View1(*EntityManager, Entity1);
	FMassEntityView View2(*EntityManager, Entity2);

	// HasElement with bIncludeSparse=true should check per-entity
	INFO("Entity1.HasElement should return true for sparse fragment");
	CHECK(View1.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::Yes));
	INFO("Entity2.HasElement should return false for sparse fragment");
	CHECK_FALSE(View2.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::Yes));

	// HasElement with bIncludeSparse=false should only check archetype
	INFO("Entity1.HasElement(false) should return false");
	CHECK_FALSE(View1.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::No));
	INFO("Entity2.HasElement(false) should return false");
	CHECK_FALSE(View2.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::No));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.HasElement.Tag", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity1 = EntityManager->CreateEntity(FloatsArchetype);
	const FMassEntityHandle Entity2 = EntityManager->CreateEntity(FloatsArchetype);

	// Add sparse tag to only Entity1
	EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entity1);

	FMassEntityView View1(*EntityManager, Entity1);
	FMassEntityView View2(*EntityManager, Entity2);

	// HasElement should work with sparse tags too
	INFO("Entity1 HasElement should detect sparse tag");
	CHECK(View1.HasElement<FTestTag_SparseA>(UE::Mass::EIncludeSparseElements::Yes));
	INFO("Entity2 HasElement should not detect sparse tag");
	CHECK_FALSE(View2.HasElement<FTestTag_SparseA>(UE::Mass::EIncludeSparseElements::Yes));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.GetSparseFragmentDataPtr", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add sparse fragment and set value
	FTestFragment_SparseInt& SparseData = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
	SparseData.Value = 42;

	FMassEntityView EntityView(*EntityManager, Entity);

	// Test const version
	const FTestFragment_SparseInt* ConstFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("GetSparseFragmentDataPtr (const) should return valid pointer");
	REQUIRE(ConstFragment != nullptr);
	INFO("Value should be accessible");
	CHECK(ConstFragment->Value == 42);

	// Test mutable version
	FTestFragment_SparseInt* MutableFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("GetSparseFragmentDataPtr (mutable) should return valid pointer");
	REQUIRE(MutableFragment != nullptr);

	// Modify value
	MutableFragment->Value = 100;

	// Verify modification persisted
	const FTestFragment_SparseInt* VerifyFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Modified value should persist");
	CHECK(VerifyFragment->Value == 100);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.GetSparseFragmentDataPtr.NonExistent", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	FMassEntityView EntityView(*EntityManager, Entity);

	// Get pointer to sparse fragment that doesn't exist
	const FTestFragment_SparseFloat* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();
	INFO("GetSparseFragmentDataPtr should return null for non-existent fragment");
	CHECK(Fragment == nullptr);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.MultipleTypes", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);

	// Add multiple sparse types
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entity);
	EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entity);

	FMassEntityView EntityView(*EntityManager, Entity);

	// Verify HasElement works for all
	INFO("HasElement detects sparse int");
	CHECK(EntityView.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::Yes));
	INFO("HasElement detects sparse float");
	CHECK(EntityView.HasElement<FTestFragment_SparseFloat>(UE::Mass::EIncludeSparseElements::Yes));
	INFO("HasElement detects sparse tag");
	CHECK(EntityView.HasElement<FTestTag_SparseA>(UE::Mass::EIncludeSparseElements::Yes));

	// Verify GetSparseFragmentDataPtr works for all fragment types
	INFO("Can get sparse int pointer");
	CHECK(EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>() != nullptr);
	INFO("Can get sparse float pointer");
	CHECK(EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>() != nullptr);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.AfterArchetypeChange", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add sparse fragment
	FTestFragment_SparseInt& SparseData = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
	SparseData.Value = 777;

	// Get initial view
	FMassEntityView View1(*EntityManager, Entity);
	INFO("View1 detects sparse fragment");
	CHECK(View1.HasElement<FTestFragment_SparseInt>());

	// Change archetype by adding regular fragment
	EntityManager->AddFragmentToEntity(Entity, FTestFragment_Float::StaticStruct());

	// Old view should be invalidated
	INFO("Old view invalidated after archetype change");
	CHECK_FALSE(View1.IsValid());

	// Create new view
	FMassEntityView View2(*EntityManager, Entity);
	INFO("New view is valid");
	CHECK(View2.IsValid());

	// Verify sparse fragment still accessible
	INFO("Sparse fragment persisted through archetype change");
	CHECK(View2.HasElement<FTestFragment_SparseInt>());

	const FTestFragment_SparseInt* Fragment = View2.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Can still get sparse fragment pointer");
	REQUIRE(Fragment != nullptr);
	INFO("Sparse fragment data preserved");
	CHECK(Fragment->Value == 777);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.RemoveAndCheck", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add sparse fragment
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);

	FMassEntityView EntityView(*EntityManager, Entity);
	INFO("Fragment exists before removal");
	CHECK(EntityView.HasElement<FTestFragment_SparseInt>());

	// Remove sparse fragment
	EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	// Note: EntityView doesn't get invalidated by sparse element changes (only archetype changes)
	INFO("View still valid after sparse removal");
	CHECK(EntityView.IsValid());

	// But HasElement should return false now
	INFO("HasElement returns false after removal");
	CHECK_FALSE(EntityView.HasElement<FTestFragment_SparseInt>());

	// And pointer should be null
	INFO("GetSparseFragmentDataPtr returns null after removal");
	CHECK(EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>() == nullptr);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.HasElement.bIncludeSparse", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Entity has regular Int fragment (from archetype) but not Float
	INFO("Entity has Int fragment in archetype");
	CHECK(EntityManager->GetFragmentDataPtr<FTestFragment_Int>(Entity) != nullptr);
	INFO("Entity doesn't have Float fragment in archetype");
	CHECK_FALSE(EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity) != nullptr);

	// Add sparse fragment
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entity);

	FMassEntityView EntityView(*EntityManager, Entity);

	// HasElement with bIncludeSparse=true should detect sparse fragment
	INFO("HasElement(true) detects sparse fragment");
	CHECK(EntityView.HasElement<FTestFragment_SparseFloat>(UE::Mass::EIncludeSparseElements::Yes));

	// HasElement with bIncludeSparse=false should NOT detect sparse fragment
	INFO("HasElement(false) doesn't detect sparse fragment");
	CHECK_FALSE(EntityView.HasElement<FTestFragment_SparseFloat>(UE::Mass::EIncludeSparseElements::No));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.AcrossDifferentArchetypes", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle EntityInts = EntityManager->CreateEntity(IntsArchetype);
	const FMassEntityHandle EntityFloats = EntityManager->CreateEntity(FloatsArchetype);

	// Add same sparse fragment to entities in different archetypes
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(EntityInts);
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(EntityFloats);

	FMassEntityView ViewInts(*EntityManager, EntityInts);
	FMassEntityView ViewFloats(*EntityManager, EntityFloats);

	// Both should detect the sparse fragment
	INFO("Ints archetype entity has sparse fragment");
	CHECK(ViewInts.HasElement<FTestFragment_SparseInt>());
	INFO("Floats archetype entity has sparse fragment");
	CHECK(ViewFloats.HasElement<FTestFragment_SparseInt>());

	// Both should provide access to the fragment
	INFO("Ints entity can get sparse fragment");
	CHECK(ViewInts.GetSparseFragmentDataPtr<FTestFragment_SparseInt>() != nullptr);
	INFO("Floats entity can get sparse fragment");
	CHECK(ViewFloats.GetSparseFragmentDataPtr<FTestFragment_SparseInt>() != nullptr);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.MixedRegularAndSparse", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsIntsArchetype);

	// Entity has regular Float and Int fragments from archetype
	// Add sparse fragment
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);

	FMassEntityView EntityView(*EntityManager, Entity);

	// Regular fragments should be accessible
	FTestFragment_Float* FloatFragment = EntityView.GetFragmentDataPtr<FTestFragment_Float>();
	FTestFragment_Int* IntFragment = EntityView.GetFragmentDataPtr<FTestFragment_Int>();
	INFO("Can access regular Float fragment");
	CHECK(FloatFragment != nullptr);
	INFO("Can access regular Int fragment");
	CHECK(IntFragment != nullptr);

	// Sparse fragment should be accessible
	FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Can access sparse fragment");
	CHECK(SparseFragment != nullptr);

	// HasFragment should detect both types
	INFO("HasFragment detects regular Float");
	CHECK(EntityView.HasFragment<FTestFragment_Float>());
	INFO("HasFragment detects regular Int");
	CHECK(EntityView.HasFragment<FTestFragment_Int>());
	INFO("HasElement detects sparse Int");
	CHECK(EntityView.HasElement<FTestFragment_SparseInt>());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.EntityView.Sparse.TryMakeView", "[Mass][Stress][EntityView][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle ValidEntity = EntityManager->CreateEntity(IntsArchetype);
	EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(ValidEntity);

	const FMassEntityHandle InvalidEntity; // Default constructor = invalid

	// TryMakeView with valid entity
	FMassEntityView ValidView = FMassEntityView::TryMakeView(*EntityManager, ValidEntity);
	INFO("TryMakeView succeeds for valid entity");
	CHECK(ValidView.IsSet());
	INFO("View detects sparse fragment");
	CHECK(ValidView.HasElement<FTestFragment_SparseInt>());

	// TryMakeView with invalid entity
	FMassEntityView InvalidView = FMassEntityView::TryMakeView(*EntityManager, InvalidEntity);
	INFO("TryMakeView returns unset view for invalid entity");
	CHECK_FALSE(InvalidView.IsSet());
}

#endif // WITH_MASSENTITY_DEBUG

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
