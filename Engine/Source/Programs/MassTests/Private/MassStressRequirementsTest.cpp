// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassRequirements.h"
#include "MassArchetypeTypes.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

#if WITH_MASSENTITY_DEBUG

//----------------------------------------------------------------------//
// FMassFragmentRequirements - Fragment Requirements Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Fragment.AddRequirement.All", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Add fragment with All presence
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);

	// Verify fragment is in RequiredAllFragments
	INFO("RequiredAllFragments should contain the added fragment");
	CHECK(Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
	INFO("RequiredAnyFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredAnyFragments().Contains<FTestFragment_Int>());
	INFO("RequiredOptionalFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Int>());
	INFO("RequiredNoneFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Int>());

	// Verify fragment requirements description
	TConstArrayView<FMassFragmentRequirementDescription> FragmentRequirements = Requirements.GetFragmentRequirements();
	INFO("Should have one fragment requirement");
	CHECK(FragmentRequirements.Num() == 1);
	INFO("Fragment type should match");
	CHECK(FragmentRequirements[0].StructType == FTestFragment_Int::StaticStruct());
	INFO("Access mode should be ReadOnly");
	CHECK(FragmentRequirements[0].AccessMode == EMassFragmentAccess::ReadOnly);
	INFO("Presence should be All");
	CHECK(FragmentRequirements[0].Presence == EMassFragmentPresence::All);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Fragment.AddRequirement.Any", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Any);

	INFO("RequiredAllFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
	INFO("RequiredAnyFragments should contain the added fragment");
	CHECK(Requirements.GetRequiredAnyFragments().Contains<FTestFragment_Int>());
	INFO("RequiredOptionalFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Int>());
	INFO("RequiredNoneFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Int>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Fragment.AddRequirement.Optional", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);

	INFO("RequiredAllFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
	INFO("RequiredAnyFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredAnyFragments().Contains<FTestFragment_Int>());
	INFO("RequiredOptionalFragments should contain the added fragment");
	CHECK(Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Int>());
	INFO("RequiredNoneFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Int>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Fragment.AddRequirement.None", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);

	INFO("RequiredAllFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
	INFO("RequiredAnyFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredAnyFragments().Contains<FTestFragment_Int>());
	INFO("RequiredOptionalFragments should not contain the fragment");
	CHECK_FALSE(Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Int>());
	INFO("RequiredNoneFragments should contain the added fragment");
	CHECK(Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Int>());

	// None presence should not add to FragmentRequirements array
	INFO("FragmentRequirements should be empty for None presence");
	CHECK(Requirements.GetFragmentRequirements().Num() == 0);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Fragment.AddRequirement.NonTemplate", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use non-template version
	Requirements.AddRequirement(FTestFragment_Float::StaticStruct(), EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);

	INFO("RequiredAllFragments should contain the added fragment");
	CHECK(Requirements.GetRequiredAllFragments().Contains(FTestFragment_Float::StaticStruct()));

	TConstArrayView<FMassFragmentRequirementDescription> FragmentRequirements = Requirements.GetFragmentRequirements();
	INFO("Should have one fragment requirement");
	CHECK(FragmentRequirements.Num() == 1);
	INFO("Access mode should be ReadWrite");
	CHECK(FragmentRequirements[0].AccessMode == EMassFragmentAccess::ReadWrite);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Fragment.AddRequirement.Multiple", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	Requirements.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::None, EMassFragmentPresence::None);

	INFO("RequiredAllFragments should contain Int");
	CHECK(Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
	INFO("RequiredOptionalFragments should contain Float");
	CHECK(Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Float>());
	INFO("RequiredNoneFragments should contain Bool");
	CHECK(Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Bool>());

	// FragmentRequirements should have 2 entries (All and Optional, not None)
	INFO("Should have two fragment requirements (excluding None)");
	CHECK(Requirements.GetFragmentRequirements().Num() == 2);
}


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Tag Requirements Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Tag.AddTagRequirement.All", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);

	INFO("RequiredAllTags should contain the added tag");
	CHECK(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("RequiredAnyTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredAnyTags().Contains<FTestTag_A>());
	INFO("RequiredNoneTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredNoneTags().Contains<FTestTag_A>());
	INFO("RequiredOptionalTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredOptionalTags().Contains<FTestTag_A>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Tag.AddTagRequirement.Any", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::Any);

	INFO("RequiredAllTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("RequiredAnyTags should contain the added tag");
	CHECK(Requirements.GetRequiredAnyTags().Contains<FTestTag_A>());
	INFO("RequiredNoneTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredNoneTags().Contains<FTestTag_A>());
	INFO("RequiredOptionalTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredOptionalTags().Contains<FTestTag_A>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Tag.AddTagRequirement.None", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::None);

	INFO("RequiredAllTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("RequiredAnyTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredAnyTags().Contains<FTestTag_A>());
	INFO("RequiredNoneTags should contain the added tag");
	CHECK(Requirements.GetRequiredNoneTags().Contains<FTestTag_A>());
	INFO("RequiredOptionalTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredOptionalTags().Contains<FTestTag_A>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Tag.AddTagRequirement.Optional", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::Optional);

	INFO("RequiredAllTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("RequiredAnyTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredAnyTags().Contains<FTestTag_A>());
	INFO("RequiredNoneTags should not contain the tag");
	CHECK_FALSE(Requirements.GetRequiredNoneTags().Contains<FTestTag_A>());
	INFO("RequiredOptionalTags should contain the added tag");
	CHECK(Requirements.GetRequiredOptionalTags().Contains<FTestTag_A>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Tag.AddTagRequirement.NonTemplate", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddTagRequirement(FTestTag_B::StaticStruct(), EMassFragmentPresence::All);

	INFO("RequiredAllTags should contain the added tag");
	CHECK(Requirements.GetRequiredAllTags().Contains(FTestTag_B::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Tag.AddTagRequirements.Batch", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	FMassTagBitSet TagBitSet;
	TagBitSet.Add<FTestTag_A>();
	TagBitSet.Add<FTestTag_B>();

	// Add batch of tags with All presence
	Requirements.AddTagRequirements<EMassFragmentPresence::All>(TagBitSet);

	INFO("RequiredAllTags should contain Tag A");
	CHECK(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("RequiredAllTags should contain Tag B");
	CHECK(Requirements.GetRequiredAllTags().Contains<FTestTag_B>());

	// Add another batch with None presence
	FMassTagBitSet NoneTagBitSet;
	NoneTagBitSet.Add<FTestTag_C>();
	Requirements.AddTagRequirements<EMassFragmentPresence::None>(NoneTagBitSet);

	INFO("RequiredNoneTags should contain Tag C");
	CHECK(Requirements.GetRequiredNoneTags().Contains<FTestTag_C>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Tag.ClearTagRequirements", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Add tags to various categories
	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
	Requirements.AddTagRequirement<FTestTag_B>(EMassFragmentPresence::Any);
	Requirements.AddTagRequirement<FTestTag_C>(EMassFragmentPresence::None);

	// Verify tags are present
	INFO("Tag A should be in AllTags");
	CHECK(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("Tag B should be in AnyTags");
	CHECK(Requirements.GetRequiredAnyTags().Contains<FTestTag_B>());
	INFO("Tag C should be in NoneTags");
	CHECK(Requirements.GetRequiredNoneTags().Contains<FTestTag_C>());

	// Clear Tag A and B
	FMassTagBitSet TagsToClear;
	TagsToClear.Add<FTestTag_A>();
	TagsToClear.Add<FTestTag_B>();
	Requirements.ClearTagRequirements(TagsToClear);

	// Verify cleared tags are gone
	INFO("Tag A should be cleared from AllTags");
	CHECK_FALSE(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("Tag B should be cleared from AnyTags");
	CHECK_FALSE(Requirements.GetRequiredAnyTags().Contains<FTestTag_B>());

	// Tag C should still be present (wasn't in clear set)
	INFO("Tag C should still be in NoneTags");
	CHECK(Requirements.GetRequiredNoneTags().Contains<FTestTag_C>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Tag.AddTagRequirement.Multiple", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
	Requirements.AddTagRequirement<FTestTag_B>(EMassFragmentPresence::Any);
	Requirements.AddTagRequirement<FTestTag_C>(EMassFragmentPresence::None);
	Requirements.AddTagRequirement<FTestTag_D>(EMassFragmentPresence::Optional);

	INFO("Tag A in AllTags");
	CHECK(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("Tag B in AnyTags");
	CHECK(Requirements.GetRequiredAnyTags().Contains<FTestTag_B>());
	INFO("Tag C in NoneTags");
	CHECK(Requirements.GetRequiredNoneTags().Contains<FTestTag_C>());
	INFO("Tag D in OptionalTags");
	CHECK(Requirements.GetRequiredOptionalTags().Contains<FTestTag_D>());
}


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Sparse Element Requirements Tests
// These tests use the unified API (AddRequirement/AddTagRequirement) with sparse elements
// to verify that the functions automatically detect and route sparse elements correctly
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.All", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use AddRequirement with a sparse fragment - should route to sparse storage
	Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);

	// Should be in sparse elements, NOT regular fragments
	INFO("RequiredAllSparseElements should contain the sparse fragment");
	CHECK(Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("RequiredAllFragments should NOT contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredAllFragments().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("RequiredAnySparseElements should not contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredAnySparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("RequiredOptionalSparseElements should not contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredOptionalSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("RequiredNoneSparseElements should not contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredNoneSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));

	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties
	INFO("HasSparseRequirements should return true");
	CHECK(Requirements.HasSparseRequirements());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.Any", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use AddRequirement with a sparse fragment - should route to sparse storage
	Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Any);

	INFO("RequiredAllSparseElements should not contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("RequiredAnySparseElements should contain the sparse fragment");
	CHECK(Requirements.GetRequiredAnySparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("RequiredAnyFragments should NOT contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredAnyFragments().Contains(FTestFragment_SparseInt::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.Optional", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use AddRequirement with a sparse fragment - should route to sparse storage
	Requirements.AddRequirement<FTestFragment_SparseFloat>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);

	INFO("RequiredAllSparseElements should not contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseFloat::StaticStruct()));
	INFO("RequiredOptionalSparseElements should contain the sparse fragment");
	CHECK(Requirements.GetRequiredOptionalSparseElements().Contains(FTestFragment_SparseFloat::StaticStruct()));
	INFO("RequiredOptionalFragments should NOT contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredOptionalFragments().Contains(FTestFragment_SparseFloat::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.None", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use AddRequirement with a sparse fragment - should route to sparse storage
	Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::None, EMassFragmentPresence::None);

	INFO("RequiredNoneSparseElements should contain the sparse fragment");
	CHECK(Requirements.GetRequiredNoneSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("RequiredNoneFragments should NOT contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredNoneFragments().Contains(FTestFragment_SparseInt::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.All", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use AddTagRequirement with a sparse tag - should route to sparse storage
	Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::All);

	INFO("RequiredAllSparseElements should contain the sparse tag");
	CHECK(Requirements.GetRequiredAllSparseElements().Contains(FTestTag_SparseA::StaticStruct()));
	INFO("RequiredAllTags should NOT contain the sparse tag");
	CHECK_FALSE(Requirements.GetRequiredAllTags().Contains(FTestTag_SparseA::StaticStruct()));

	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties
	INFO("HasSparseRequirements should return true");
	CHECK(Requirements.HasSparseRequirements());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.Any", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use AddTagRequirement with a sparse tag - should route to sparse storage
	Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::Any);

	INFO("RequiredAnySparseElements should contain the sparse tag");
	CHECK(Requirements.GetRequiredAnySparseElements().Contains(FTestTag_SparseA::StaticStruct()));
	INFO("RequiredAnyTags should NOT contain the sparse tag");
	CHECK_FALSE(Requirements.GetRequiredAnyTags().Contains(FTestTag_SparseA::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.Optional", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use AddTagRequirement with a sparse tag - should route to sparse storage
	Requirements.AddTagRequirement<FTestTag_SparseB>(EMassFragmentPresence::Optional);

	INFO("RequiredOptionalSparseElements should contain the sparse tag");
	CHECK(Requirements.GetRequiredOptionalSparseElements().Contains(FTestTag_SparseB::StaticStruct()));
	INFO("RequiredOptionalTags should NOT contain the sparse tag");
	CHECK_FALSE(Requirements.GetRequiredOptionalTags().Contains(FTestTag_SparseB::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.None", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use AddTagRequirement with a sparse tag - should route to sparse storage
	Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::None);

	INFO("RequiredNoneSparseElements should contain the sparse tag");
	CHECK(Requirements.GetRequiredNoneSparseElements().Contains(FTestTag_SparseA::StaticStruct()));
	INFO("RequiredNoneTags should NOT contain the sparse tag");
	CHECK_FALSE(Requirements.GetRequiredNoneTags().Contains(FTestTag_SparseA::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.NonTemplate", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use non-template AddRequirement with sparse fragment
	Requirements.AddRequirement(FTestFragment_SparseInt::StaticStruct(), EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);

	INFO("RequiredAllSparseElements should contain the sparse fragment");
	CHECK(Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("RequiredAllFragments should NOT contain the sparse fragment");
	CHECK_FALSE(Requirements.GetRequiredAllFragments().Contains(FTestFragment_SparseInt::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.NonTemplate", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Use non-template AddTagRequirement with sparse tag
	Requirements.AddTagRequirement(FTestTag_SparseB::StaticStruct(), EMassFragmentPresence::Any);

	INFO("RequiredAnySparseElements should contain the sparse tag");
	CHECK(Requirements.GetRequiredAnySparseElements().Contains(FTestTag_SparseB::StaticStruct()));
	INFO("RequiredAnyTags should NOT contain the sparse tag");
	CHECK_FALSE(Requirements.GetRequiredAnyTags().Contains(FTestTag_SparseB::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.UnifiedAPI.Mixed", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Mix of sparse fragments and sparse tags using unified API
	Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddRequirement<FTestFragment_SparseFloat>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::Any);
	Requirements.AddTagRequirement<FTestTag_SparseB>(EMassFragmentPresence::None);

	// All should be routed to sparse storage
	INFO("Sparse int in AllSparseElements");
	CHECK(Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("Sparse float in OptionalSparseElements");
	CHECK(Requirements.GetRequiredOptionalSparseElements().Contains(FTestFragment_SparseFloat::StaticStruct()));
	INFO("Sparse tag A in AnySparseElements");
	CHECK(Requirements.GetRequiredAnySparseElements().Contains(FTestTag_SparseA::StaticStruct()));
	INFO("Sparse tag B in NoneSparseElements");
	CHECK(Requirements.GetRequiredNoneSparseElements().Contains(FTestTag_SparseB::StaticStruct()));

	// None should be in regular storage
	INFO("Sparse int NOT in AllFragments");
	CHECK_FALSE(Requirements.GetRequiredAllFragments().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("Sparse float NOT in OptionalFragments");
	CHECK_FALSE(Requirements.GetRequiredOptionalFragments().Contains(FTestFragment_SparseFloat::StaticStruct()));
	INFO("Sparse tag A NOT in AnyTags");
	CHECK_FALSE(Requirements.GetRequiredAnyTags().Contains(FTestTag_SparseA::StaticStruct()));
	INFO("Sparse tag B NOT in NoneTags");
	CHECK_FALSE(Requirements.GetRequiredNoneTags().Contains(FTestTag_SparseB::StaticStruct()));

	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties
	INFO("HasSparseRequirements should return true");
	CHECK(Requirements.HasSparseRequirements());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Sparse.UnifiedAPI.MixedRegularAndSparse", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Add regular fragments
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	// Add regular tags
	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
	Requirements.AddTagRequirement<FTestTag_B>(EMassFragmentPresence::None);
	// Add sparse fragments using unified API
	Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddRequirement<FTestFragment_SparseFloat>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Any);
	// Add sparse tags using unified API
	Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::Optional);
	Requirements.AddTagRequirement<FTestTag_SparseB>(EMassFragmentPresence::None);

	// Regular elements should be in regular storage
	INFO("Int fragment in AllFragments");
	CHECK(Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
	INFO("Float fragment in OptionalFragments");
	CHECK(Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Float>());
	INFO("Tag A in AllTags");
	CHECK(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("Tag B in NoneTags");
	CHECK(Requirements.GetRequiredNoneTags().Contains<FTestTag_B>());

	// Sparse elements should be in sparse storage
	INFO("Sparse int in AllSparseElements");
	CHECK(Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("Sparse float in AnySparseElements");
	CHECK(Requirements.GetRequiredAnySparseElements().Contains(FTestFragment_SparseFloat::StaticStruct()));
	INFO("Sparse tag A in OptionalSparseElements");
	CHECK(Requirements.GetRequiredOptionalSparseElements().Contains(FTestTag_SparseA::StaticStruct()));
	INFO("Sparse tag B in NoneSparseElements");
	CHECK(Requirements.GetRequiredNoneSparseElements().Contains(FTestTag_SparseB::StaticStruct()));

	// Sparse elements should NOT be in regular storage
	INFO("Sparse int NOT in regular AllFragments");
	CHECK_FALSE(Requirements.GetRequiredAllFragments().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("Sparse float NOT in regular AnyFragments");
	CHECK_FALSE(Requirements.GetRequiredAnyFragments().Contains(FTestFragment_SparseFloat::StaticStruct()));
	INFO("Sparse tag A NOT in regular OptionalTags");
	CHECK_FALSE(Requirements.GetRequiredOptionalTags().Contains(FTestTag_SparseA::StaticStruct()));
	INFO("Sparse tag B NOT in regular NoneTags");
	CHECK_FALSE(Requirements.GetRequiredNoneTags().Contains(FTestTag_SparseB::StaticStruct()));

	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties
	INFO("HasSparseRequirements should return true");
	CHECK(Requirements.HasSparseRequirements());
	INFO("HasPositiveRequirements should return true");
	CHECK(Requirements.HasPositiveRequirements());
	INFO("HasNegativeRequirements should return true");
	CHECK(Requirements.HasNegativeRequirements());
	INFO("HasOptionalRequirements should return true");
	CHECK(Requirements.HasOptionalRequirements());
}


//----------------------------------------------------------------------//
// FMassFragmentRequirements - State and Validation Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.State.IsEmpty", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	INFO("Newly created requirements should be empty");
	CHECK(Requirements.IsEmpty());

	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Requirements with fragment should not be empty");
	CHECK_FALSE(Requirements.IsEmpty());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.State.HasPositiveRequirements", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Initially no positive requirements
	INFO("Empty requirements have no positive requirements");
	CHECK_FALSE(Requirements.HasPositiveRequirements());

	// Add "All" fragment - this is a positive requirement
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Requirements with All fragment have positive requirements");
	CHECK(Requirements.HasPositiveRequirements());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.State.HasNegativeRequirements", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	INFO("Empty requirements have no negative requirements");
	CHECK_FALSE(Requirements.HasNegativeRequirements());

	// Add "None" fragment - this is a negative requirement
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Requirements with None fragment have negative requirements");
	CHECK(Requirements.HasNegativeRequirements());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.State.HasOptionalRequirements", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	INFO("Empty requirements have no optional requirements");
	CHECK_FALSE(Requirements.HasOptionalRequirements());

	// Add "Optional" fragment
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Requirements with Optional fragment have optional requirements");
	CHECK(Requirements.HasOptionalRequirements());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.State.HasSparseRequirements", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	INFO("Empty requirements have no sparse requirements");
	CHECK_FALSE(Requirements.HasSparseRequirements());

	// Use unified API - AddRequirement should detect sparse element and route appropriately
	Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Requirements with sparse element have sparse requirements");
	CHECK(Requirements.HasSparseRequirements());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.State.Reset", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Add various requirements - use unified API for sparse element
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
	Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Requirements should not be empty before reset");
	CHECK_FALSE(Requirements.IsEmpty());

	Requirements.Reset();
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Requirements should be empty after reset");
	CHECK(Requirements.IsEmpty());
	INFO("Should not have positive requirements after reset");
	CHECK_FALSE(Requirements.HasPositiveRequirements());
	INFO("Should not have sparse requirements after reset");
	CHECK_FALSE(Requirements.HasSparseRequirements());
}


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Archetype Matching Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Match.AllFragments", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	// IntsArchetype has FTestFragment_Int but not FTestFragment_Float
	INFO("IntsArchetype should not match (missing Float)");
	CHECK_FALSE(Requirements.DoesArchetypeMatchRequirements(IntsArchetype));

	// FloatsIntsArchetype has both
	INFO("FloatsIntsArchetype should match");
	CHECK(Requirements.DoesArchetypeMatchRequirements(FloatsIntsArchetype));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Match.AnyFragments", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Any);
	Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Any);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	// IntsArchetype has Int
	INFO("IntsArchetype should match (has Int)");
	CHECK(Requirements.DoesArchetypeMatchRequirements(IntsArchetype));

	// FloatsArchetype has Float
	INFO("FloatsArchetype should match (has Float)");
	CHECK(Requirements.DoesArchetypeMatchRequirements(FloatsArchetype));

	// EmptyArchetype has neither
	INFO("EmptyArchetype should not match (has neither)");
	CHECK_FALSE(Requirements.DoesArchetypeMatchRequirements(EmptyArchetype));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Match.NoneFragments", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::None, EMassFragmentPresence::None);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	// IntsArchetype has Int but not Float - should match
	INFO("IntsArchetype should match (has Int, no Float)");
	CHECK(Requirements.DoesArchetypeMatchRequirements(IntsArchetype));

	// FloatsIntsArchetype has both Int and Float - should not match (has Float which is excluded)
	INFO("FloatsIntsArchetype should not match (has excluded Float)");
	CHECK_FALSE(Requirements.DoesArchetypeMatchRequirements(FloatsIntsArchetype));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Match.Tags", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	// Create archetype with tag
	FMassArchetypeCompositionDescriptor CompositionWithTag;
	CompositionWithTag.Add<FTestFragment_Int>();
	CompositionWithTag.Add<FTestTag_A>();

	FMassArchetypeHandle ArchetypeWithTag = EntityManager->CreateArchetype(CompositionWithTag, {});

	FMassFragmentRequirements Requirements(EntityManager);
	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Archetype with tag should match");
	CHECK(Requirements.DoesArchetypeMatchRequirements(ArchetypeWithTag));

	INFO("IntsArchetype (no tag) should not match");
	CHECK_FALSE(Requirements.DoesArchetypeMatchRequirements(IntsArchetype));
}


//----------------------------------------------------------------------//
// FMassFragmentRequirements - AddElementRequirement Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Element.AddElementRequirement.Fragment", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// AddElementRequirement should route fragments to AddRequirement
	Requirements.AddElementRequirement(FTestFragment_Int::StaticStruct(), EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Fragment should be added to RequiredAllFragments");
	CHECK(Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Element.AddElementRequirement.Tag", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// AddElementRequirement should route tags to AddTagRequirement
	Requirements.AddElementRequirement(FTestTag_A::StaticStruct(), EMassFragmentAccess::None, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Tag should be added to RequiredAllTags");
	CHECK(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
}


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Chunk Fragment Requirements Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.ChunkFragment.AddChunkRequirement.All", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("RequiredAllChunkFragments should contain the chunk fragment");
	CHECK(Requirements.GetRequiredAllChunkFragments().Contains<FTestChunkFragment_Int>());

	TConstArrayView<FMassFragmentRequirementDescription> ChunkRequirements = Requirements.GetChunkFragmentRequirements();
	INFO("Should have one chunk fragment requirement");
	CHECK(ChunkRequirements.Num() == 1);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.ChunkFragment.AddChunkRequirement.Optional", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddChunkRequirement<FTestChunkFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("RequiredOptionalChunkFragments should contain the chunk fragment");
	CHECK(Requirements.GetRequiredOptionalChunkFragments().Contains<FTestChunkFragment_Float>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.ChunkFragment.AddChunkRequirement.None", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("RequiredNoneChunkFragments should contain the chunk fragment");
	CHECK(Requirements.GetRequiredNoneChunkFragments().Contains<FTestChunkFragment_Int>());

	// None presence should not add to ChunkFragmentRequirements array
	INFO("ChunkFragmentRequirements should be empty for None presence");
	CHECK(Requirements.GetChunkFragmentRequirements().Num() == 0);
}


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Shared Fragment Requirements Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.SharedFragment.AddSharedRequirement.All", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("RequiredAllSharedFragments should contain the shared fragment");
	CHECK(Requirements.GetRequiredAllSharedFragments().Contains<FTestSharedFragment_Int>());

	TConstArrayView<FMassFragmentRequirementDescription> SharedRequirements = Requirements.GetSharedFragmentRequirements();
	INFO("Should have one shared fragment requirement");
	CHECK(SharedRequirements.Num() == 1);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.SharedFragment.AddSharedRequirement.Optional", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddSharedRequirement<FTestSharedFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("RequiredOptionalSharedFragments should contain the shared fragment");
	CHECK(Requirements.GetRequiredOptionalSharedFragments().Contains<FTestSharedFragment_Float>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.SharedFragment.AddSharedRequirement.None", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("RequiredNoneSharedFragments should contain the shared fragment");
	CHECK(Requirements.GetRequiredNoneSharedFragments().Contains<FTestSharedFragment_Int>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.ConstSharedFragment.AddConstSharedRequirement.All", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("RequiredAllConstSharedFragments should contain the const shared fragment");
	CHECK(Requirements.GetRequiredAllConstSharedFragments().Contains<FTestConstSharedFragment_Int>());

	TConstArrayView<FMassFragmentRequirementDescription> ConstSharedRequirements = Requirements.GetConstSharedFragmentRequirements();
	INFO("Should have one const shared fragment requirement");
	CHECK(ConstSharedRequirements.Num() == 1);
	// Const shared fragments are always ReadOnly
	INFO("Access mode should be ReadOnly");
	CHECK(ConstSharedRequirements[0].AccessMode == EMassFragmentAccess::ReadOnly);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.ConstSharedFragment.AddConstSharedRequirement.Optional", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddConstSharedRequirement<FTestConstSharedFragment_Float>(EMassFragmentPresence::Optional);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("RequiredOptionalConstSharedFragments should contain the const shared fragment");
	CHECK(Requirements.GetRequiredOptionalConstSharedFragments().Contains<FTestConstSharedFragment_Float>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.ConstSharedFragment.AddConstSharedRequirement.None", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	Requirements.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::None);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("RequiredNoneConstSharedFragments should contain the const shared fragment");
	CHECK(Requirements.GetRequiredNoneConstSharedFragments().Contains<FTestConstSharedFragment_Int>());
}


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Comprehensive Mixed Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Mixed.Comprehensive", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Add various requirement types - use unified API for sparse element
	Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
	Requirements.AddTagRequirement<FTestTag_B>(EMassFragmentPresence::None);
	Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	// Verify all requirements are tracked
	INFO("Requirements should not be empty");
	CHECK_FALSE(Requirements.IsEmpty());
	INFO("Should have positive requirements");
	CHECK(Requirements.HasPositiveRequirements());
	INFO("Should have negative requirements (None tag)");
	CHECK(Requirements.HasNegativeRequirements());
	INFO("Should have optional requirements");
	CHECK(Requirements.HasOptionalRequirements());
	INFO("Should have sparse requirements");
	CHECK(Requirements.HasSparseRequirements());

	// Verify specific requirements
	INFO("Fragment Int in All");
	CHECK(Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
	INFO("Fragment Float in Optional");
	CHECK(Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Float>());
	INFO("Tag A in All");
	CHECK(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("Tag B in None");
	CHECK(Requirements.GetRequiredNoneTags().Contains<FTestTag_B>());
	INFO("Sparse Int in All");
	CHECK(Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("Chunk Int in All");
	CHECK(Requirements.GetRequiredAllChunkFragments().Contains<FTestChunkFragment_Int>());
	INFO("Shared Int in All");
	CHECK(Requirements.GetRequiredAllSharedFragments().Contains<FTestSharedFragment_Int>());
	INFO("ConstShared Int in All");
	CHECK(Requirements.GetRequiredAllConstSharedFragments().Contains<FTestConstSharedFragment_Int>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Requirements.Mixed.Chaining", "[Mass][Stress][Requirements][Debug]")
{
	REQUIRE(EntityManager);

	FMassFragmentRequirements Requirements(EntityManager);

	// Test method chaining - use unified API for sparse element
	Requirements
		.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All)
		.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional)
		.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All)
		.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
	Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

	INFO("Fragment Int added via chaining");
	CHECK(Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
	INFO("Fragment Float added via chaining");
	CHECK(Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Float>());
	INFO("Tag A added via chaining");
	CHECK(Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
	INFO("Sparse Int added via chaining");
	CHECK(Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
}

#endif // WITH_MASSENTITY_DEBUG

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
