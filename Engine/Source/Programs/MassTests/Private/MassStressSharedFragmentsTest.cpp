// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassEntityManager.h"
#include "MassEntityTypes.h"
#include "MassArchetypeData.h"
#include "MassArchetypeTypes.h"
#include "MassEntityView.h"
#include "Algo/RandomShuffle.h"
#include "MassObserverNotificationTypes.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

//----------------------------------------------------------------------//
// FMassArchetypeData::SetSharedFragmentsData Tests
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.Basic", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialValue = 100;
	constexpr int32 NewValue = 200;

	// Create an archetype with shared fragment
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Verify initial shared fragment value
	const FTestSharedFragment_Int* InitialFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	INFO("Entity should have shared fragment after creation");
	REQUIRE(InitialFragment != nullptr);
	INFO("Initial shared fragment value should match");
	CHECK(InitialFragment->Value == InitialValue);

	// Get archetype data to call SetSharedFragmentsData
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	// Prepare override values
	TArray<FSharedStruct> Overrides;
	Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewValue));

	// Call SetSharedFragmentsData
	ArchetypeData.SetSharedFragmentsData(Entity, MakeArrayView(Overrides));

	// Verify the entity is still valid and in the same archetype
	INFO("Entity should still be valid after SetSharedFragmentsData");
	CHECK(EntityManager->IsEntityValid(Entity));
	INFO("Entity should remain in the same archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == Archetype);

	// Verify the shared fragment value changed
	const FTestSharedFragment_Int* UpdatedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	INFO("Entity should still have shared fragment after SetSharedFragmentsData");
	REQUIRE(UpdatedFragment != nullptr);
	INFO("Shared fragment value should be updated");
	CHECK(UpdatedFragment->Value == NewValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.MultipleEntitiesDifferentChunks", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ValueA = 10;
	constexpr int32 ValueB = 20;
	constexpr int32 ValueC = 30;

	// Create multiple entities with different initial shared fragment values
	FMassArchetypeSharedFragmentValues SharedValuesA;
	SharedValuesA.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueA));
	SharedValuesA.Sort();

	FMassArchetypeSharedFragmentValues SharedValuesB;
	SharedValuesB.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueB));
	SharedValuesB.Sort();

	FMassEntityHandle EntityA = EntityManager->CreateEntity(FloatsArchetype, SharedValuesA);
	FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype, SharedValuesB);

	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(EntityA);
	INFO("Both entities should be in same archetype");
	CHECK(EntityManager->GetArchetypeForEntity(EntityB) == Archetype);

	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	// Initially there should be 2 chunks (one for each shared value)
	INFO("Should have 2 chunks initially (one per shared value)");
	CHECK(ArchetypeData.GetChunkCount() == 2);

	// Move EntityA to a new shared value (ValueC)
	TArray<FSharedStruct> OverridesForA;
	OverridesForA.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueC));
	ArchetypeData.SetSharedFragmentsData<FSharedStruct>(EntityA, OverridesForA);

	// Verify EntityA moved to new chunk with ValueC
	const FTestSharedFragment_Int* FragmentA = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityA);
	INFO("EntityA should have shared fragment");
	REQUIRE(FragmentA != nullptr);
	INFO("EntityA's shared fragment should have new value");
	CHECK(FragmentA->Value == ValueC);

	// Verify EntityB still has original value
	const FTestSharedFragment_Int* FragmentB = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityB);
	INFO("EntityB should have shared fragment");
	REQUIRE(FragmentB != nullptr);
	INFO("EntityB's shared fragment should be unchanged");
	CHECK(FragmentB->Value == ValueB);

	// Should now have 3 chunks (ValueA empty but still exists, ValueB, ValueC)
	// Note: Empty chunks may or may not be cleaned up depending on implementation
	INFO("Chunk count should be at least 2");
	CHECK(ArchetypeData.GetChunkCount() >= 2);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.PreservesFragmentData", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialSharedValue = 100;
	constexpr int32 NewSharedValue = 200;
	constexpr float TestFloatValue = 42.5f;

	// Create entity with both regular fragment data and shared fragment
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialSharedValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Set a specific value in the regular fragment
	FTestFragment_Float* FloatFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity);
	INFO("Entity should have float fragment");
	REQUIRE(FloatFragment != nullptr);
	FloatFragment->Value = TestFloatValue;

	// Get archetype and call SetSharedFragmentsData
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	TArray<FSharedStruct> Overrides;
	Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewSharedValue));
	ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, Overrides);

	// Verify regular fragment data is preserved
	FloatFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity);
	INFO("Entity should still have float fragment after move");
	REQUIRE(FloatFragment != nullptr);
	INFO("Float fragment value should be preserved after SetSharedFragmentsData");
	CHECK(FloatFragment->Value == TestFloatValue);

	// Verify shared fragment was updated
	const FTestSharedFragment_Int* SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	INFO("Entity should have shared fragment");
	REQUIRE(SharedFragment != nullptr);
	INFO("Shared fragment should have new value");
	CHECK(SharedFragment->Value == NewSharedValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.MultipleSharedFragments", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialIntValue = 100;
	constexpr float InitialFloatValue = 1.5f;
	constexpr int32 NewIntValue = 200;
	constexpr float NewFloatValue = 2.5f;

	// Create entity with multiple shared fragments
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialIntValue));
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Float>(InitialFloatValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Verify initial values
	const FTestSharedFragment_Int* IntFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	const FTestSharedFragment_Float* FloatFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
	INFO("Entity should have int shared fragment");
	REQUIRE(IntFragment != nullptr);
	INFO("Entity should have float shared fragment");
	REQUIRE(FloatFragment != nullptr);
	INFO("Initial int shared fragment value");
	CHECK(IntFragment->Value == InitialIntValue);
	INFO("Initial float shared fragment value");
	CHECK(FloatFragment->Value == InitialFloatValue);

	// Get archetype and update both shared fragments
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	TArray<FSharedStruct> Overrides;
	Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewIntValue));
	Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Float>(NewFloatValue));
	ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, Overrides);

	// Verify both shared fragments updated
	IntFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	FloatFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
	INFO("Entity should still have int shared fragment");
	REQUIRE(IntFragment != nullptr);
	INFO("Entity should still have float shared fragment");
	REQUIRE(FloatFragment != nullptr);
	INFO("Int shared fragment should have new value");
	CHECK(IntFragment->Value == NewIntValue);
	INFO("Float shared fragment should have new value");
	CHECK(FloatFragment->Value == NewFloatValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.PartialUpdate", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialIntValue = 100;
	constexpr float InitialFloatValue = 1.5f;
	constexpr int32 NewIntValue = 200;

	// Create entity with multiple shared fragments
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialIntValue));
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Float>(InitialFloatValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Get archetype and update only the int shared fragment
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	TArray<FSharedStruct> Overrides;
	Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewIntValue));
	// Note: Not adding FTestSharedFragment_Float to overrides
	ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, Overrides);

	// Verify int shared fragment updated
	const FTestSharedFragment_Int* IntFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	INFO("Entity should have int shared fragment");
	REQUIRE(IntFragment != nullptr);
	INFO("Int shared fragment should have new value");
	CHECK(IntFragment->Value == NewIntValue);

	// Verify float shared fragment kept its original value
	const FTestSharedFragment_Float* FloatFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
	INFO("Entity should have float shared fragment");
	REQUIRE(FloatFragment != nullptr);
	INFO("Float shared fragment should keep original value");
	CHECK(FloatFragment->Value == InitialFloatValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.MoveToExistingChunk", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ValueA = 10;
	constexpr int32 ValueB = 20;

	// Create two entities with different shared values - this creates two chunks
	FMassArchetypeSharedFragmentValues SharedValuesA;
	SharedValuesA.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueA));
	SharedValuesA.Sort();

	FMassArchetypeSharedFragmentValues SharedValuesB;
	FSharedStruct SharedValueB = FSharedStruct::Make<FTestSharedFragment_Int>(ValueB);
	SharedValuesB.Add(SharedValueB);
	SharedValuesB.Sort();

	FMassEntityHandle EntityA = EntityManager->CreateEntity(FloatsArchetype, SharedValuesA);
	FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype, SharedValuesB);

	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(EntityA);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	const int32 InitialChunkCount = ArchetypeData.GetChunkCount();
	INFO("Should have 2 chunks initially");
	CHECK(InitialChunkCount == 2);

	// Move EntityA to the same shared value as EntityB
	TArray<FSharedStruct> Overrides;
	//Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueB));
	Overrides.Add(SharedValueB);
	ArchetypeData.SetSharedFragmentsData<FSharedStruct>(EntityA, Overrides);

	// Verify EntityA now has ValueB
	const FTestSharedFragment_Int* FragmentA = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityA);
	INFO("EntityA should have shared fragment");
	REQUIRE(FragmentA != nullptr);
	INFO("EntityA should now have ValueB");
	CHECK(FragmentA->Value == ValueB);

	// Verify both entities have the same shared fragment instance (they're in the same chunk)
	const FTestSharedFragment_Int* FragmentB = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityB);
	INFO("EntityB should have shared fragment");
	REQUIRE(FragmentB != nullptr);
	INFO("EntityA and EntityB should share the same shared fragment instance");
	CHECK(FragmentA == FragmentB);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.WithSparseElements", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialSharedValue = 100;
	constexpr int32 NewSharedValue = 200;
	constexpr int32 SparseIntValue = 42;

	// Create entity with shared fragment
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialSharedValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Add a sparse element to the entity
	FTestFragment_SparseInt& SparseInt = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
	SparseInt.Value = SparseIntValue;

	// Verify sparse element exists
	INFO("Entity should have sparse element before SetSharedFragmentsData");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	FMassEntityView EntityView(*EntityManager, Entity);

	const FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Sparse fragment should exist");
	REQUIRE(SparseFragment != nullptr);
	INFO("Sparse fragment initial value");
	CHECK(SparseFragment->Value == SparseIntValue);

	// Get archetype and call SetSharedFragmentsData
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	TArray<FSharedStruct> Overrides;
	Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewSharedValue));
	// this operation invalidates the entity view
	ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, Overrides);

	// Verify sparse element is preserved after the move
	INFO("Entity should still have sparse element after SetSharedFragmentsData");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	FMassEntityView EntityView2(*EntityManager, Entity);
	SparseFragment = EntityView2.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Sparse fragment should still exist after move");
	REQUIRE(SparseFragment != nullptr);
	INFO("Sparse fragment value should be preserved");
	CHECK(SparseFragment->Value == SparseIntValue);

	// Verify shared fragment was updated
	const FTestSharedFragment_Int* SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	INFO("Shared fragment should exist");
	REQUIRE(SharedFragment != nullptr);
	INFO("Shared fragment should have new value");
	CHECK(SharedFragment->Value == NewSharedValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.ChunkBoundary", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ValueA = 10;
	constexpr int32 ValueB = 20;

	// Create a batch with shared values — the actual archetype will include the shared fragment type
	FMassArchetypeSharedFragmentValues SharedValuesA;
	SharedValuesA.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueA));
	SharedValuesA.Sort();

	// Create an initial batch to discover the actual archetype, then size subsequent creation
	TArray<FMassEntityHandle> InitEntities;
	EntityManager->BatchCreateEntities(FloatsArchetype, SharedValuesA, 1, InitEntities);
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(InitEntities[0]);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);
	const int32 EntitiesPerChunk = ArchetypeData.GetNumEntitiesPerChunk();

	// Destroy the probe so it doesn't affect counts
	EntityManager->DestroyEntity(InitEntities[0]);

	const int32 NumEntitiesToCreate = FMath::FloorToInt32(static_cast<float>(EntitiesPerChunk) * 1.5f); // Create 1.5 chunks worth

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, SharedValuesA, NumEntitiesToCreate, Entities);

	// Re-fetch archetype data (may have been reallocated after destroy+recreate)
	Archetype = EntityManager->GetArchetypeForEntity(Entities[0]);
	FMassArchetypeData& ArchetypeDataRef = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	// Move an entity from the middle of the batch to a new shared value
	const int32 EntityToMoveIndex = NumEntitiesToCreate / 2;
	FMassEntityHandle EntityToMove = Entities[EntityToMoveIndex];

	TArray<FSharedStruct> Overrides;
	Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueB));
	ArchetypeDataRef.SetSharedFragmentsData<FSharedStruct>(EntityToMove, Overrides);

	// Verify the moved entity has the new value
	const FTestSharedFragment_Int* MovedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityToMove);
	INFO("Moved entity should have shared fragment");
	REQUIRE(MovedFragment != nullptr);
	INFO("Moved entity should have new shared value");
	CHECK(MovedFragment->Value == ValueB);

	// Verify other entities still have original value
	for (int32 i = 0; i < NumEntitiesToCreate; ++i)
	{
		if (i != EntityToMoveIndex)
		{
			const FTestSharedFragment_Int* Fragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entities[i]);
			INFO("Other entity should have shared fragment");
			REQUIRE(Fragment != nullptr);
			INFO("Other entity should have original shared value");
			CHECK(Fragment->Value == ValueA);
		}
	}

	// Total entity count should be preserved
	INFO("Total entity count should be preserved");
	CHECK(ArchetypeDataRef.GetNumEntities() == NumEntitiesToCreate);
}


//----------------------------------------------------------------------//
// FMassArchetypeData::SetSharedFragmentsData Tests - Const Shared Fragments
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetConstSharedFragmentsData.Basic", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialValue = 100;
	constexpr int32 NewValue = 200;

	// Create an archetype with const shared fragment
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Verify initial const shared fragment value
	const FTestConstSharedFragment_Int* InitialFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	INFO("Entity should have const shared fragment after creation");
	REQUIRE(InitialFragment != nullptr);
	INFO("Initial const shared fragment value should match");
	CHECK(InitialFragment->Value == InitialValue);

	// Get archetype data to call SetSharedFragmentsData
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	// Prepare override values
	TArray<FConstSharedStruct> Overrides;
	Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewValue));

	// Call SetSharedFragmentsData with const shared fragment
	ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

	// Verify the entity is still valid and in the same archetype
	INFO("Entity should still be valid after SetSharedFragmentsData");
	CHECK(EntityManager->IsEntityValid(Entity));
	INFO("Entity should remain in the same archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == Archetype);

	// Verify the const shared fragment value changed
	const FTestConstSharedFragment_Int* UpdatedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	INFO("Entity should still have const shared fragment after SetSharedFragmentsData");
	REQUIRE(UpdatedFragment != nullptr);
	INFO("Const shared fragment value should be updated");
	CHECK(UpdatedFragment->Value == NewValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetConstSharedFragmentsData.MultipleEntitiesDifferentChunks", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ValueA = 10;
	constexpr int32 ValueB = 20;
	constexpr int32 ValueC = 30;

	// Create multiple entities with different initial const shared fragment values
	FMassArchetypeSharedFragmentValues SharedValuesA;
	SharedValuesA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueA));
	SharedValuesA.Sort();

	FMassArchetypeSharedFragmentValues SharedValuesB;
	SharedValuesB.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueB));
	SharedValuesB.Sort();

	FMassEntityHandle EntityA = EntityManager->CreateEntity(FloatsArchetype, SharedValuesA);
	FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype, SharedValuesB);

	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(EntityA);
	INFO("Both entities should be in same archetype");
	CHECK(EntityManager->GetArchetypeForEntity(EntityB) == Archetype);

	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	// Initially there should be 2 chunks (one for each const shared value)
	INFO("Should have 2 chunks initially (one per const shared value)");
	CHECK(ArchetypeData.GetChunkCount() == 2);

	// Move EntityA to a new const shared value (ValueC)
	TArray<FConstSharedStruct> OverridesForA;
	OverridesForA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueC));
	ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(EntityA, OverridesForA);

	// Verify EntityA moved to new chunk with ValueC
	const FTestConstSharedFragment_Int* FragmentA = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityA);
	INFO("EntityA should have const shared fragment");
	REQUIRE(FragmentA != nullptr);
	INFO("EntityA's const shared fragment should have new value");
	CHECK(FragmentA->Value == ValueC);

	// Verify EntityB still has original value
	const FTestConstSharedFragment_Int* FragmentB = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityB);
	INFO("EntityB should have const shared fragment");
	REQUIRE(FragmentB != nullptr);
	INFO("EntityB's const shared fragment should be unchanged");
	CHECK(FragmentB->Value == ValueB);

	// Should now have at least 2 chunks
	INFO("Chunk count should be at least 2");
	CHECK(ArchetypeData.GetChunkCount() >= 2);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetConstSharedFragmentsData.PreservesFragmentData", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialSharedValue = 100;
	constexpr int32 NewSharedValue = 200;
	constexpr float TestFloatValue = 42.5f;

	// Create entity with both regular fragment data and const shared fragment
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialSharedValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Set a specific value in the regular fragment
	FTestFragment_Float* FloatFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity);
	INFO("Entity should have float fragment");
	REQUIRE(FloatFragment != nullptr);
	FloatFragment->Value = TestFloatValue;

	// Get archetype and call SetSharedFragmentsData
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	TArray<FConstSharedStruct> Overrides;
	Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewSharedValue));
	ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

	// Verify regular fragment data is preserved
	FloatFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity);
	INFO("Entity should still have float fragment after move");
	REQUIRE(FloatFragment != nullptr);
	INFO("Float fragment value should be preserved after SetSharedFragmentsData");
	CHECK(FloatFragment->Value == TestFloatValue);

	// Verify const shared fragment was updated
	const FTestConstSharedFragment_Int* SharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	INFO("Entity should have const shared fragment");
	REQUIRE(SharedFragment != nullptr);
	INFO("Const shared fragment should have new value");
	CHECK(SharedFragment->Value == NewSharedValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetConstSharedFragmentsData.MultipleConstSharedFragments", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialIntValue = 100;
	constexpr float InitialFloatValue = 1.5f;
	constexpr int32 NewIntValue = 200;
	constexpr float NewFloatValue = 2.5f;

	// Create entity with multiple const shared fragments
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialIntValue));
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(InitialFloatValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Verify initial values
	const FTestConstSharedFragment_Int* IntFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	const FTestConstSharedFragment_Float* FloatFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);
	INFO("Entity should have int const shared fragment");
	REQUIRE(IntFragment != nullptr);
	INFO("Entity should have float const shared fragment");
	REQUIRE(FloatFragment != nullptr);
	INFO("Initial int const shared fragment value");
	CHECK(IntFragment->Value == InitialIntValue);
	INFO("Initial float const shared fragment value");
	CHECK(FloatFragment->Value == InitialFloatValue);

	// Get archetype and update both const shared fragments
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	TArray<FConstSharedStruct> Overrides;
	Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewIntValue));
	Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(NewFloatValue));
	ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

	// Verify both const shared fragments updated
	IntFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	FloatFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);
	INFO("Entity should still have int const shared fragment");
	REQUIRE(IntFragment != nullptr);
	INFO("Entity should still have float const shared fragment");
	REQUIRE(FloatFragment != nullptr);
	INFO("Int const shared fragment should have new value");
	CHECK(IntFragment->Value == NewIntValue);
	INFO("Float const shared fragment should have new value");
	CHECK(FloatFragment->Value == NewFloatValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetConstSharedFragmentsData.PartialUpdate", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialIntValue = 100;
	constexpr float InitialFloatValue = 1.5f;
	constexpr int32 NewIntValue = 200;

	// Create entity with multiple const shared fragments
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialIntValue));
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(InitialFloatValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Get archetype and update only the int const shared fragment
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	TArray<FConstSharedStruct> Overrides;
	Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewIntValue));
	// Note: Not adding FTestConstSharedFragment_Float to overrides
	ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

	// Verify int const shared fragment updated
	const FTestConstSharedFragment_Int* IntFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	INFO("Entity should have int const shared fragment");
	REQUIRE(IntFragment != nullptr);
	INFO("Int const shared fragment should have new value");
	CHECK(IntFragment->Value == NewIntValue);

	// Verify float const shared fragment kept its original value
	const FTestConstSharedFragment_Float* FloatFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);
	INFO("Entity should have float const shared fragment");
	REQUIRE(FloatFragment != nullptr);
	INFO("Float const shared fragment should keep original value");
	CHECK(FloatFragment->Value == InitialFloatValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetConstSharedFragmentsData.MoveToExistingChunk", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ValueA = 10;
	constexpr int32 ValueB = 20;

	// Create two entities with different const shared values - this creates two chunks
	FMassArchetypeSharedFragmentValues SharedValuesA;
	SharedValuesA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueA));
	SharedValuesA.Sort();

	FMassArchetypeSharedFragmentValues SharedValuesB;
	FConstSharedStruct SharedValueB = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueB);
	SharedValuesB.Add(SharedValueB);
	SharedValuesB.Sort();

	FMassEntityHandle EntityA = EntityManager->CreateEntity(FloatsArchetype, SharedValuesA);
	FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype, SharedValuesB);

	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(EntityA);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	const int32 InitialChunkCount = ArchetypeData.GetChunkCount();
	INFO("Should have 2 chunks initially");
	CHECK(InitialChunkCount == 2);

	// Move EntityA to the same const shared value as EntityB
	TArray<FConstSharedStruct> Overrides;
	Overrides.Add(SharedValueB);
	ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(EntityA, Overrides);

	// Verify EntityA now has ValueB
	const FTestConstSharedFragment_Int* FragmentA = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityA);
	INFO("EntityA should have const shared fragment");
	REQUIRE(FragmentA != nullptr);
	INFO("EntityA should now have ValueB");
	CHECK(FragmentA->Value == ValueB);

	// Verify both entities have the same const shared fragment instance (they're in the same chunk)
	const FTestConstSharedFragment_Int* FragmentB = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityB);
	INFO("EntityB should have const shared fragment");
	REQUIRE(FragmentB != nullptr);
	INFO("EntityA and EntityB should share the same const shared fragment instance");
	CHECK(FragmentA == FragmentB);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetConstSharedFragmentsData.WithSparseElements", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialSharedValue = 100;
	constexpr int32 NewSharedValue = 200;
	constexpr int32 SparseIntValue = 42;

	// Create entity with const shared fragment
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialSharedValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Add a sparse element to the entity
	FTestFragment_SparseInt& SparseInt = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
	SparseInt.Value = SparseIntValue;

	// Verify sparse element exists
	INFO("Entity should have sparse element before SetSharedFragmentsData");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	FMassEntityView EntityView(*EntityManager, Entity);

	const FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Sparse fragment should exist");
	REQUIRE(SparseFragment != nullptr);
	INFO("Sparse fragment initial value");
	CHECK(SparseFragment->Value == SparseIntValue);

	// Get archetype and call SetSharedFragmentsData
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	TArray<FConstSharedStruct> Overrides;
	Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewSharedValue));
	// this operation invalidates the entity view
	ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

	// Verify sparse element is preserved after the move
	INFO("Entity should still have sparse element after SetSharedFragmentsData");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	FMassEntityView NewEntityView(*EntityManager, Entity);
	SparseFragment = NewEntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Sparse fragment should still exist after move");
	REQUIRE(SparseFragment != nullptr);
	INFO("Sparse fragment value should be preserved");
	CHECK(SparseFragment->Value == SparseIntValue);

	// Verify const shared fragment was updated
	const FTestConstSharedFragment_Int* SharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	INFO("Const shared fragment should exist");
	REQUIRE(SharedFragment != nullptr);
	INFO("Const shared fragment should have new value");
	CHECK(SharedFragment->Value == NewSharedValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetConstSharedFragmentsData.ChunkBoundary", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 ValueA = 10;
	constexpr int32 ValueB = 20;

	// Create a batch with const shared values — the actual archetype will include the const shared fragment type
	FMassArchetypeSharedFragmentValues SharedValuesA;
	SharedValuesA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueA));
	SharedValuesA.Sort();

	// Create an initial batch to discover the actual archetype, then size subsequent creation
	TArray<FMassEntityHandle> InitEntities;
	EntityManager->BatchCreateEntities(FloatsArchetype, SharedValuesA, 1, InitEntities);
	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(InitEntities[0]);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);
	const int32 EntitiesPerChunk = ArchetypeData.GetNumEntitiesPerChunk();

	// Destroy the probe so it doesn't affect counts
	EntityManager->DestroyEntity(InitEntities[0]);

	const int32 NumEntitiesToCreate = FMath::FloorToInt32(static_cast<float>(EntitiesPerChunk) * 1.5f); // Create 1.5 chunks worth

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, SharedValuesA, NumEntitiesToCreate, Entities);

	// Re-fetch archetype data (may have been reallocated after destroy+recreate)
	Archetype = EntityManager->GetArchetypeForEntity(Entities[0]);
	FMassArchetypeData& ArchetypeDataRef = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	// Move an entity from the middle of the batch to a new const shared value
	const int32 EntityToMoveIndex = NumEntitiesToCreate / 2;
	FMassEntityHandle EntityToMove = Entities[EntityToMoveIndex];

	TArray<FConstSharedStruct> Overrides;
	Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueB));
	ArchetypeDataRef.SetSharedFragmentsData<FConstSharedStruct>(EntityToMove, Overrides);

	// Verify the moved entity has the new value
	const FTestConstSharedFragment_Int* MovedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityToMove);
	INFO("Moved entity should have const shared fragment");
	REQUIRE(MovedFragment != nullptr);
	INFO("Moved entity should have new const shared value");
	CHECK(MovedFragment->Value == ValueB);

	// Verify other entities still have original value
	for (int32 i = 0; i < NumEntitiesToCreate; ++i)
	{
		if (i != EntityToMoveIndex)
		{
			const FTestConstSharedFragment_Int* Fragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entities[i]);
			INFO("Other entity should have const shared fragment");
			REQUIRE(Fragment != nullptr);
			INFO("Other entity should have original const shared value");
			CHECK(Fragment->Value == ValueA);
		}
	}

	// Total entity count should be preserved
	INFO("Total entity count should be preserved");
	CHECK(ArchetypeDataRef.GetNumEntities() == NumEntitiesToCreate);
}


//----------------------------------------------------------------------//
// FMassArchetypeData::SetSharedFragmentsData Tests - Mixed Shared/Const
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.MixedSharedAndConst", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialSharedIntValue = 100;
	constexpr int32 InitialConstIntValue = 200;
	constexpr int32 NewSharedIntValue = 150;
	constexpr int32 NewConstIntValue = 250;

	// Create entity with both shared and const shared fragments
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialSharedIntValue));
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialConstIntValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	// Verify initial values
	const FTestSharedFragment_Int* SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	const FTestConstSharedFragment_Int* ConstFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	INFO("Entity should have shared fragment");
	REQUIRE(SharedFragment != nullptr);
	INFO("Entity should have const shared fragment");
	REQUIRE(ConstFragment != nullptr);
	INFO("Initial shared fragment value");
	CHECK(SharedFragment->Value == InitialSharedIntValue);
	INFO("Initial const shared fragment value");
	CHECK(ConstFragment->Value == InitialConstIntValue);

	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	// Update only the mutable shared fragment
	{
		TArray<FSharedStruct> SharedOverrides;
		SharedOverrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewSharedIntValue));
		ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, SharedOverrides);
	}

	// Verify shared fragment updated, const unchanged
	SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	ConstFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	INFO("Entity should still have shared fragment");
	REQUIRE(SharedFragment != nullptr);
	INFO("Entity should still have const shared fragment");
	REQUIRE(ConstFragment != nullptr);
	INFO("Shared fragment should have new value");
	CHECK(SharedFragment->Value == NewSharedIntValue);
	INFO("Const shared fragment should be unchanged");
	CHECK(ConstFragment->Value == InitialConstIntValue);

	// Now update only the const shared fragment
	{
		TArray<FConstSharedStruct> ConstOverrides;
		ConstOverrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewConstIntValue));
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, ConstOverrides);
	}

	// Verify both are now updated
	SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	ConstFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	INFO("Entity should still have shared fragment");
	REQUIRE(SharedFragment != nullptr);
	INFO("Entity should still have const shared fragment");
	REQUIRE(ConstFragment != nullptr);
	INFO("Shared fragment should retain its value");
	CHECK(SharedFragment->Value == NewSharedIntValue);
	INFO("Const shared fragment should have new value");
	CHECK(ConstFragment->Value == NewConstIntValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.SharedFragments.SetSharedFragmentsData.MixedPreservesOtherType", "[Mass][Stress][SharedFragments]")
{
	REQUIRE(EntityManager);

	constexpr int32 SharedIntValue = 100;
	constexpr float SharedFloatValue = 1.5f;
	constexpr int32 ConstIntValue = 200;
	constexpr float ConstFloatValue = 2.5f;
	constexpr int32 NewSharedIntValue = 150;
	constexpr int32 NewConstIntValue = 250;

	// Create entity with multiple shared and const shared fragments
	FMassArchetypeSharedFragmentValues SharedValues;
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(SharedIntValue));
	SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Float>(SharedFloatValue));
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ConstIntValue));
	SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(ConstFloatValue));
	SharedValues.Sort();

	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

	FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
	FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

	// Update only one mutable shared fragment (Int), keeping Float unchanged
	{
		TArray<FSharedStruct> SharedOverrides;
		SharedOverrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewSharedIntValue));
		ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, SharedOverrides);
	}

	// Verify only the Int shared fragment changed
	const FTestSharedFragment_Int* SharedInt = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	const FTestSharedFragment_Float* SharedFloat = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
	const FTestConstSharedFragment_Int* ConstInt = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	const FTestConstSharedFragment_Float* ConstFloat = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);

	INFO("SharedInt should exist");
	REQUIRE(SharedInt != nullptr);
	INFO("SharedFloat should exist");
	REQUIRE(SharedFloat != nullptr);
	INFO("ConstInt should exist");
	REQUIRE(ConstInt != nullptr);
	INFO("ConstFloat should exist");
	REQUIRE(ConstFloat != nullptr);

	INFO("SharedInt should have new value");
	CHECK(SharedInt->Value == NewSharedIntValue);
	INFO("SharedFloat should be unchanged");
	CHECK(SharedFloat->Value == SharedFloatValue);
	INFO("ConstInt should be unchanged");
	CHECK(ConstInt->Value == ConstIntValue);
	INFO("ConstFloat should be unchanged");
	CHECK(ConstFloat->Value == ConstFloatValue);

	// Update only one const shared fragment (Int), keeping Float unchanged
	{
		TArray<FConstSharedStruct> ConstOverrides;
		ConstOverrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewConstIntValue));
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, ConstOverrides);
	}

	// Re-fetch and verify
	SharedInt = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
	SharedFloat = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
	ConstInt = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
	ConstFloat = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);

	INFO("SharedInt should still exist");
	REQUIRE(SharedInt != nullptr);
	INFO("SharedFloat should still exist");
	REQUIRE(SharedFloat != nullptr);
	INFO("ConstInt should still exist");
	REQUIRE(ConstInt != nullptr);
	INFO("ConstFloat should still exist");
	REQUIRE(ConstFloat != nullptr);

	INFO("SharedInt should retain value");
	CHECK(SharedInt->Value == NewSharedIntValue);
	INFO("SharedFloat should still be unchanged");
	CHECK(SharedFloat->Value == SharedFloatValue);
	INFO("ConstInt should have new value");
	CHECK(ConstInt->Value == NewConstIntValue);
	INFO("ConstFloat should still be unchanged");
	CHECK(ConstFloat->Value == ConstFloatValue);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
