// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTTypes.h"
#include "MassArchetypeData.h"
#include "MassEntityBuilder.h"
#include "MassEntityManager.h"
#include "Mass/EntityMacros.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

// to be enabled once this functionality gets implemented
#define WITH_ENTITY_MOVING_AFFECTING_COMPOSITION 0

namespace UE::Mass::LLT
{
#if WITH_MASSENTITY_DEBUG

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.Storage.ConfigureType", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	UE::Mass::FSparseElementsStorage Storage;

	// Configure custom chunk size
	UE::Mass::FSparseElementsStorage::FTypeConfig Config(FTestFragment_SparseInt::StaticStruct());
	// change the default chunk size, for testing purposes.
	Config.SetElementsPerChunk(64);

	Storage.ConfigureType(FTestFragment_SparseInt::StaticStruct(), Config);

	// Create entities beyond one chunk
	constexpr int32 NumEntities = 200;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse elements - should use configured chunk size
	for (const FMassEntityHandle& Entity : Entities)
	{
		FStructView View = Storage.AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		INFO("Added element with custom chunk size");
		CHECK(View.IsValid());
	}

	// Verify we can access all of them
	for (const FMassEntityHandle& Entity : Entities)
	{
		FStructView View = Storage.GetMutableElementDataForEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		INFO("Element accessible with custom chunk size");
		CHECK(View.IsValid());
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.Storage.OnEntitiesDestroyed", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();

	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse elements to all
	for (const FMassEntityHandle& Entity : Entities)
	{
		FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		INFO("Element added successfully");
		CHECK(View.IsValid());
	}

	// Destroy subset - test the bug fix (return -> continue)
	TArray<FMassEntityHandle> EntitiesToDestroy = {Entities[2], Entities[5], Entities[7]};
	EntityManager->BatchDestroyEntities(EntitiesToDestroy);

	// Verify destroyed entities no longer have elements
	for (const FMassEntityHandle& Entity : EntitiesToDestroy)
	{
		FStructView View = Storage.GetMutableElementDataForEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		INFO("Destroyed entity has no sparse element");
		CHECK_FALSE(View.IsValid());
	}

	// Verify remaining entities still have elements (validates bug fix)
	for (int32 Index = 0; Index < NumEntities; ++Index)
	{
		if (!EntitiesToDestroy.Contains(Entities[Index]))
		{
			FStructView View = Storage.GetMutableElementDataForEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
			INFO("Remaining entity still has sparse element");
			CHECK(View.IsValid());
		}
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.MultipleTypes", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add 3 different sparse types
	FStructView IntView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	FStructView FloatView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
	FStructView TagView = EntityManager->AddSparseElementToEntity(Entity, FTestTag_SparseA::StaticStruct());

	INFO("Sparse int added");
	CHECK(IntView.IsValid());
	INFO("Sparse float added");
	CHECK(FloatView.IsValid());
	INFO("Added sparse tag view is invalid");
	CHECK(TagView.IsValid() == false);

	// Verify all are present
	FMassEntityView EntityView(*EntityManager, Entity);
	INFO("Has sparse int");
	CHECK(EntityView.HasElement<FTestFragment_SparseInt>());
	INFO("Has sparse float");
	CHECK(EntityView.HasElement<FTestFragment_SparseFloat>());
	INFO("Has sparse tag");
	CHECK(EntityView.HasElement<FTestTag_SparseA>());

	// Remove one, verify others remain
	EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseFloat::StaticStruct());

	INFO("Still has sparse int");
	CHECK(EntityView.HasElement<FTestFragment_SparseInt>());
	INFO("No longer has sparse float");
	CHECK_FALSE(EntityView.HasElement<FTestFragment_SparseFloat>());
	INFO("Still has sparse tag");
	CHECK(EntityView.HasElement<FTestTag_SparseA>());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.RemoveNonExistent", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Try to remove element that was never added
	EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	// Add, remove, try to remove again
	FStructView _ = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.DataPersistence", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 20;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse fragments with unique values
	for (int32 Index = 0; Index < NumEntities; ++Index)
	{
		FStructView View = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
		FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
		Fragment->Value = Index * 100;
	}

	// Move entities to different archetype
	EntityManager->BatchChangeFragmentCompositionForEntities(
		{FMassArchetypeEntityCollection(IntsArchetype, Entities, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates)},
		FMassFragmentBitSet(*FTestFragment_Float::StaticStruct()),
		{}
	);

	// Verify data persisted through move
	for (int32 Index = 0; Index < NumEntities; ++Index)
	{
		FMassEntityView EntityView(*EntityManager, Entities[Index]);
		const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		INFO("Fragment still exists after move");
		REQUIRE(Fragment != nullptr);
		INFO("Fragment value preserved");
		CHECK(Fragment->Value == Index * 100);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.CrossChunkBatch", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
	constexpr int32 NumChunks = 5;
	const int32 NumEntities = NumChunks * EntitiesPerChunk;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Select entities scattered across all chunks
	TArray<FMassEntityHandle> SelectedEntities;
	for (int32 ChunkIdx = 0; ChunkIdx < NumChunks; ++ChunkIdx)
	{
		// Pick 3 entities from each chunk
		for (int32 InChunkIdx = 0; InChunkIdx < 3 && (ChunkIdx * EntitiesPerChunk + InChunkIdx) < NumEntities; ++InChunkIdx)
		{
			SelectedEntities.Add(Entities[ChunkIdx * EntitiesPerChunk + InChunkIdx]);
		}
	}

	// Batch add across chunks
	FMassArchetypeEntityCollection Collection(IntsArchetype, SelectedEntities, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());

	// Verify all have the element
	for (const FMassEntityHandle& Entity : SelectedEntities)
	{
		FMassEntityView EntityView(*EntityManager, Entity);
		INFO("Entity has sparse element");
		CHECK(EntityView.HasElement<FTestFragment_SparseInt>());
	}

	// Batch remove across chunks
	EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());

	// Verify all removed
	for (const FMassEntityHandle& Entity : SelectedEntities)
	{
		FMassEntityView EntityView(*EntityManager, Entity);
		INFO("Entity no longer has sparse element");
		CHECK_FALSE(EntityView.HasElement<FTestFragment_SparseInt>());
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.Storage.ChunkBoundary", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	UE::Mass::FSparseElementsStorage Storage;

	// Test accesses at chunk boundaries (default 128 elements per chunk)
	constexpr int32 BoundaryIndices[] = {0, 127, 128, 255, 256, 383, 384};

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, 500, Entities);

	// Add sparse elements at boundary indices
	for (int32 BoundaryIdx : BoundaryIndices)
	{
		if (BoundaryIdx < Entities.Num())
		{
			FStructView View = Storage.AddElementToEntity(Entities[BoundaryIdx], FTestFragment_SparseInt::StaticStruct());
			INFO("Boundary add succeeded");
			CHECK(View.IsValid());

			FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
			Fragment->Value = BoundaryIdx;
		}
	}

	// Verify all boundary elements accessible
	for (int32 BoundaryIdx : BoundaryIndices)
	{
		if (BoundaryIdx < Entities.Num())
		{
			FConstStructView View = Storage.GetElementDataForEntity(Entities[BoundaryIdx], FTestFragment_SparseInt::StaticStruct());
			INFO("Boundary element accessible");
			CHECK(View.IsValid());

			const FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
			INFO("Fragment pointer valid");
			REQUIRE(Fragment != nullptr);
			INFO("Boundary element value correct");
			CHECK(Fragment->Value == BoundaryIdx);
		}
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.BatchEmpty", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	TArray<FMassEntityHandle> EmptyArray;
	FMassArchetypeEntityCollection EmptyCollection;

	// Should not crash with empty input
	EntityManager->BatchAddSparseElementToEntities({EmptyCollection}, FTestFragment_SparseInt::StaticStruct());
	EntityManager->BatchRemoveSparseElementFromEntities({EmptyCollection}, FTestFragment_SparseInt::StaticStruct());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ComplexQuery", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 20;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Group 1: Has SparseInt only (entities 0-4)
	for (int32 Index = 0; Index < 5; ++Index)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
	}

	// Group 2: Has SparseFloat only (entities 5-9)
	for (int32 Index = 5; Index < 10; ++Index)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseFloat::StaticStruct());
	}

	// Group 3: Has both (entities 10-14)
	for (int32 Index = 10; Index < 15; ++Index)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
		FStructView __ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseFloat::StaticStruct());
	}

	// Group 4: Has neither (entities 15-19)

	// Query 1: Entities with both sparse elements
	{
		FMassEntityQuery Query(EntityManager);
		// required since Queries don't support sparse-only requirements
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddSparseRequirement<FTestFragment_SparseInt>();
		Query.AddSparseRequirement<FTestFragment_SparseFloat>();

		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
		int32 Count = 0;
		Query.ForEachEntityChunk(ExecutionContext, [&Count](FMassExecutionContext& Context)
		{
			for (FMassExecutionContext::FSparseEntityIterator It = Context.CreateSparseEntityIterator(); It; ++It)
			{
				++Count;
			}
		});

		INFO("Query with both sparse elements");
		CHECK(Count == 5);
	}

	// Query 2: Entities with SparseInt but NOT SparseFloat
	{
		FMassEntityQuery Query(EntityManager);
		// required since Queries don't support sparse-only requirements
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddSparseRequirement<FTestFragment_SparseInt>();
		Query.AddSparseRequirement<FTestFragment_SparseFloat>(EMassFragmentPresence::None);

		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
		int32 Count = 0;
		Query.ForEachEntityChunk(ExecutionContext, [&Count](FMassExecutionContext& Context)
		{
			for (FMassExecutionContext::FSparseEntityIterator It = Context.CreateSparseEntityIterator(); It; ++It)
			{
				++Count;
			}
		});

		INFO("Query with SparseInt, without SparseFloat");
		CHECK(Count == 5);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.MemoryFragmentation", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	UE::Mass::FSparseElementsStorage Storage;

	constexpr int32 NumEntities = 1000;
	constexpr int32 NumCycles = 10;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Repeatedly add and remove to create fragmentation
	for (int32 Cycle = 0; Cycle < NumCycles; ++Cycle)
	{
		// Add to all
		for (const FMassEntityHandle& Entity : Entities)
		{
			FStructView _ = Storage.AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		}

		// Remove from half (creates holes)
		for (int32 Index = 0; Index < NumEntities / 2; ++Index)
		{
			Storage.RemoveElementFromEntity<FTestFragment_SparseInt>(Entities[Index]);
		}

		// Re-add to different half
		for (int32 Index = 0; Index < NumEntities / 2; ++Index)
		{
			FStructView _ = Storage.AddElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
		}
	}

	// Final verification - all should have the element
	for (const FMassEntityHandle& Entity : Entities)
	{
		FStructView View = Storage.GetMutableElementDataForEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		INFO("Entity has sparse element after fragmentation");
		CHECK(View.IsValid());
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.OccupationFiltering", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 200;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add to odd indices only
	for (int32 Index = 1; Index < NumEntities; Index += 2)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
	}

	// Query should only find odd indices
	FMassEntityQuery Query(EntityManager);
	Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
	Query.AddSparseRequirement<FTestFragment_SparseInt>();

	FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
	TSet<int32> FoundIndices;

	Query.ForEachEntityChunk(ExecutionContext, [&FoundIndices](FMassExecutionContext& Context)
	{
		for (FMassExecutionContext::FSparseEntityIterator It = Context.CreateSparseEntityIterator(); It; ++It)
		{
			FoundIndices.Add(Context.GetEntities()[It].Index);
		}
	});

	// Verify iterator correctly skipped even indices
	int32 ExpectedCount = 0;
	for (int32 Index = 0; Index < NumEntities; ++Index)
	{
		const bool bShouldHave = (Index % 2 == 1);
		const bool bActuallyHas = FoundIndices.Contains(Entities[Index].Index);
		INFO("Occupation mask filtering correct");
		CHECK(bActuallyHas == bShouldHave);
		if (bShouldHave) ++ExpectedCount;
	}

	INFO("Found expected number of sparse elements");
	CHECK(FoundIndices.Num() == ExpectedCount);
}

// CRITICAL: Test that UStruct destructors are called (validates bug fix #2)
// Note: This test requires a special fragment type that tracks construction/destruction
// If FTestFragment_SparseInt doesn't have suitable tracking, this test may need modification
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.DestructorValidation", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();

	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse fragments - they should be properly constructed
	for (const FMassEntityHandle& Entity : Entities)
	{
		FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		INFO("Fragment added");
		CHECK(View.IsValid());

		// Modify to ensure it's a real instance
		FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
		Fragment->Value = 42;
	}

	// Remove half - DestroyStruct should be called
	for (int32 Index = 0; Index < NumEntities / 2; ++Index)
	{
		EntityManager->RemoveSparseElementFromEntity<FTestFragment_SparseInt>(Entities[Index]);

		// Verify it's actually gone
		FStructView View = Storage.GetMutableElementDataForEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
		INFO("Fragment no longer exists");
		CHECK_FALSE(View.IsValid());
	}

	// Destroy remaining entities - should call destructors via OnEntitiesDestroyed
	TArray<FMassEntityHandle> RemainingEntities;
	for (int32 Index = NumEntities / 2; Index < NumEntities; ++Index)
	{
		RemainingEntities.Add(Entities[Index]);
	}
	EntityManager->BatchDestroyEntities(RemainingEntities);

	// Verify all destroyed
	for (const FMassEntityHandle& Entity : RemainingEntities)
	{
		FStructView View = Storage.GetMutableElementDataForEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		INFO("Destroyed entity fragment removed");
		CHECK_FALSE(View.IsValid());
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.InstanceCountTracking", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	FSparseElementsStorage Storage;

	// Initially empty
	INFO("Initial count is zero");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == 0u);
	INFO("Initially has no elements");
	CHECK_FALSE(Storage.HasAnyElementsOfType<FTestFragment_SparseInt>());

	constexpr int32 NumEntities = 50;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add to half
	for (int32 i = 0; i < NumEntities / 2; ++i)
	{
		FStructView _ = Storage.AddElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
	}

	INFO("Count after adding half");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == 25u);
	INFO("Has elements after adding");
	CHECK(Storage.HasAnyElementsOfType<FTestFragment_SparseInt>());

	// Remove quarter
	for (int32 i = 0; i < NumEntities / 4; ++i)
	{
		Storage.RemoveElementFromEntity<FTestFragment_SparseInt>(Entities[i]);
	}

	INFO("Count after removing quarter");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == 13u);

	// Remove all
	for (int32 i = 0; i < NumEntities; ++i)
	{
		Storage.RemoveElementFromEntity<FTestFragment_SparseInt>(Entities[i]);
	}

	INFO("Count after removing all");
	CHECK(Storage.GetNumElementsOfType<FTestFragment_SparseInt>() == 0u);
	INFO("No elements after removing all");
	CHECK_FALSE(Storage.HasAnyElementsOfType<FTestFragment_SparseInt>());
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.AddRemoveAddCycle", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	for (int32 i = 0; i < NumEntities; ++i)
	{
		const FMassEntityHandle Entity = Entities[i];

		// First Add: Set initial value
		{
			FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
			INFO("First add succeeded");
			CHECK(View.IsValid());

			FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
			Fragment->Value = 100 + i;
		}

		// Verify first add
		{
			FMassEntityView EntityView(*EntityManager, Entity);
			const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
			INFO("Fragment exists after first add");
			REQUIRE(Fragment != nullptr);
			INFO("First add value correct");
			CHECK(Fragment->Value == 100 + i);
		}

		// Remove
		EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

		// Verify removal
		{
			FMassEntityView EntityView(*EntityManager, Entity);
			const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
			INFO("Fragment removed");
			CHECK(Fragment == nullptr);
		}

		// Second Add: Set different value to ensure fresh memory
		{
			FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
			INFO("Second add succeeded");
			CHECK(View.IsValid());

			FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
			// Value should be default-initialized (0), not the old value (100 + i)
			INFO("Second add has fresh default value, not old data");
			CHECK(Fragment->Value == 0);

			// Set new value
			Fragment->Value = 200 + i;
		}

		// Verify second add has new value
		{
			FMassEntityView EntityView(*EntityManager, Entity);
			const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
			INFO("Fragment exists after second add");
			REQUIRE(Fragment != nullptr);
			INFO("Second add value correct");
			CHECK(Fragment->Value == 200 + i);
			INFO("Second add value different from first add");
			CHECK(Fragment->Value != 100 + i);
		}
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.AddRemoveAddBoundaries", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	// Test at specific indices that cross chunk boundaries (default 128 per chunk)
	constexpr int32 BoundaryIndices[] = { 0, 1, 126, 127, 128, 129, 255, 256, 383, 384 };
	constexpr int32 MaxIndex = 500;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, MaxIndex, Entities);

	for (int32 BoundaryIdx : BoundaryIndices)
	{
		if (BoundaryIdx >= Entities.Num())
			continue;

		const FMassEntityHandle Entity = Entities[BoundaryIdx];

		// Cycle 1: Add with value A
		{
			FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
			INFO("Boundary add 1 succeeded");
			CHECK(View.IsValid());

			FTestFragment_SparseFloat* Fragment = View.GetPtr<FTestFragment_SparseFloat>();
			Fragment->Value = static_cast<float>((BoundaryIdx + 1) * 10);
		}

		// Verify value A
		{
			FConstStructView View = EntityManager->GetSparseElementDataForEntity(
				FTestFragment_SparseFloat::StaticStruct(), Entity);
			INFO("Fragment accessible at boundary");
			CHECK(View.IsValid());
			INFO("Boundary value A correct");
			CHECK(View.Get<FTestFragment_SparseFloat>().Value == static_cast<float>((BoundaryIdx + 1) * 10));
		}

		// Remove at boundary
		EntityManager->RemoveSparseElementFromEntity(
			Entity, FTestFragment_SparseFloat::StaticStruct());

		// Verify removed
		{
			FConstStructView View = EntityManager->GetSparseElementDataForEntity(
				FTestFragment_SparseFloat::StaticStruct(), Entity);
			INFO("Fragment removed at boundary");
			CHECK_FALSE(View.IsValid());
		}

		// Cycle 2: Add with value B
		{
			FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
			INFO("Boundary add 2 succeeded");
			CHECK(View.IsValid());

			FTestFragment_SparseFloat* Fragment = View.GetPtr<FTestFragment_SparseFloat>();
			// Should be default-initialized
			INFO("Boundary second add has fresh default value");
			CHECK(Fragment->Value == 0.0f);

			Fragment->Value = static_cast<float>((BoundaryIdx + 1) * 20); // Different value
		}

		// Verify value B (different from value A)
		{
			FConstStructView View = EntityManager->GetSparseElementDataForEntity(
				FTestFragment_SparseFloat::StaticStruct(), Entity);
			INFO("Fragment accessible after second add");
			CHECK(View.IsValid());
			INFO("Boundary value B correct");
			CHECK(View.Get<FTestFragment_SparseFloat>().Value == static_cast<float>((BoundaryIdx + 1) * 20));
			INFO("Boundary value B different from A");
			CHECK(View.Get<FTestFragment_SparseFloat>().Value != static_cast<float>((BoundaryIdx + 1) * 10));
		}
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.AddRemoveAddTags", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 5;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

	for (const FMassEntityHandle& Entity : Entities)
	{
		// Add sparse tag
		{
			FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestTag_SparseA::StaticStruct());
			// Tags return invalid view (no data)
			INFO("Tag add returns empty view");
			CHECK_FALSE(View.IsValid());
		}

		// Verify tag present
		{
			FMassEntityView EntityView(*EntityManager, Entity);
			INFO("Tag present after add");
			CHECK(EntityView.HasElement<FTestTag_SparseA>());
		}

		// Remove tag
		EntityManager->RemoveSparseElementFromEntity(Entity, FTestTag_SparseA::StaticStruct());

		// Verify tag removed
		{
			FMassEntityView EntityView(*EntityManager, Entity);
			INFO("Tag removed");
			CHECK_FALSE(EntityView.HasElement<FTestTag_SparseA>());
		}

		// Add tag again
		{
			FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestTag_SparseA::StaticStruct());
			INFO("Tag second add returns empty view");
			CHECK_FALSE(View.IsValid());
		}

		// Verify tag present again
		{
			FMassEntityView EntityView(*EntityManager, Entity);
			INFO("Tag present after second add");
			CHECK(EntityView.HasElement<FTestTag_SparseA>());
		}
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.AddRemoveAddMultipleTypes", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add two different sparse types
	{
		FStructView IntView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		IntView.Get<FTestFragment_SparseInt>().Value = 100;

		FStructView FloatView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
		FloatView.Get<FTestFragment_SparseFloat>().Value = 50.0f;
	}

	// Verify both present
	{
		FMassEntityView EntityView(*EntityManager, Entity);
		INFO("Has SparseInt");
		CHECK(EntityView.HasElement<FTestFragment_SparseInt>());
		INFO("Has SparseFloat");
		CHECK(EntityView.HasElement<FTestFragment_SparseFloat>());
	}

	// Remove only SparseInt
	EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	// Verify SparseInt removed but SparseFloat remains
	{
		FMassEntityView EntityView(*EntityManager, Entity);
		INFO("SparseInt removed");
		CHECK_FALSE(EntityView.HasElement<FTestFragment_SparseInt>());
		INFO("SparseFloat still present");
		CHECK(EntityView.HasElement<FTestFragment_SparseFloat>());

		const FTestFragment_SparseFloat* FloatFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();
		INFO("SparseFloat accessible");
		REQUIRE(FloatFragment != nullptr);
		INFO("SparseFloat value unchanged");
		CHECK(FloatFragment->Value == 50.0f);
	}

	// Add SparseInt again with different value
	{
		FStructView IntView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		INFO("SparseInt re-added");
		CHECK(IntView.IsValid());

		FTestFragment_SparseInt* IntFragment = IntView.GetPtr<FTestFragment_SparseInt>();
		INFO("Re-added SparseInt has fresh default value");
		CHECK(IntFragment->Value == 0);

		IntFragment->Value = 200; // Different from original 100
	}

	// Verify both present with correct values
	{
		FMassEntityView EntityView(*EntityManager, Entity);
		INFO("Both types present");
		CHECK(EntityView.HasElement<FTestFragment_SparseInt>());
		CHECK(EntityView.HasElement<FTestFragment_SparseFloat>());

		const FTestFragment_SparseInt* IntFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		const FTestFragment_SparseFloat* FloatFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();

		INFO("SparseInt accessible");
		REQUIRE(IntFragment != nullptr);
		INFO("SparseFloat accessible");
		REQUIRE(FloatFragment != nullptr);

		INFO("SparseInt has new value");
		CHECK(IntFragment->Value == 200);
		INFO("SparseInt different from original");
		CHECK(IntFragment->Value != 100);
		INFO("SparseFloat value unchanged");
		CHECK(FloatFragment->Value == 50.0f);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.AddRemoveAddWithArchetypeChange", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add sparse element to entity in IntsArchetype
	{
		FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		View.Get<FTestFragment_SparseInt>().Value = 111;
	}

	// Verify
	{
		INFO("Entity in IntsArchetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);

		FMassEntityView EntityView(*EntityManager, Entity);
		const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		INFO("Sparse element present");
		REQUIRE(Fragment != nullptr);
		INFO("Value correct");
		CHECK(Fragment->Value == 111);
	}

	// Remove sparse element
	{
		EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	}

	// Change archetype by adding regular fragment
	{
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_Float::StaticStruct());
		INFO("Entity moved to FloatsIntsArchetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsIntsArchetype);
	}

	// Add sparse element again (now in different archetype)
	{
		FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		INFO("Sparse element added after archetype change");
		CHECK(View.IsValid());

		FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
		INFO("Fresh default value after archetype change");
		CHECK(Fragment->Value == 0);

		Fragment->Value = 222;
	}

	// Verify in new archetype
	{
		INFO("Still in FloatsIntsArchetype");
		CHECK(EntityManager->GetArchetypeForEntity(Entity) == FloatsIntsArchetype);

		FMassEntityView EntityView(*EntityManager, Entity);
		const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		INFO("Sparse element present in new archetype");
		REQUIRE(Fragment != nullptr);
		INFO("New value correct");
		CHECK(Fragment->Value == 222);
		INFO("Value different from original");
		CHECK(Fragment->Value != 111);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.BatchAddFragmentInstances.PureSparse", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 20;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Create payload with sparse fragment instances
	UE::Mass::TMultiArray<FTestFragment_SparseInt, FTestFragment_SparseFloat> Payload;
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FTestFragment_SparseInt SparseInt;
		SparseInt.Value = i * 100;

		FTestFragment_SparseFloat SparseFloat;
		SparseFloat.Value = static_cast<float>(i * 5);

		Payload.Add(SparseInt, SparseFloat);
	}

	// Create entity collections with payload
	TArray<FStructArrayView> GenericPayload;
	GenericPayload.Reserve(Payload.GetNumArrays());
	Payload.GetAsGenericMultiArray(GenericPayload);

	TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
	FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(
		*EntityManager, Entities, FMassArchetypeEntityCollection::FoldDuplicates,
		FMassGenericPayloadView(GenericPayload), EntityCollections);

	// Build fragments bitset
	FMassFragmentBitSet FragmentsAffected;
	FragmentsAffected.Add(FTestFragment_SparseInt::StaticStruct());
	FragmentsAffected.Add(FTestFragment_SparseFloat::StaticStruct());

	// Execute batch add
	EntityManager->BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected);

	// Verify all entities have sparse fragments with correct data
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FMassEntityView EntityView(*EntityManager, Entities[i]);

		const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		const FTestFragment_SparseFloat* SparseFloat = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();

		INFO("SparseInt added");
		REQUIRE(SparseInt != nullptr);
		INFO("SparseFloat added");
		REQUIRE(SparseFloat != nullptr);

		INFO("SparseInt value correct");
		CHECK(SparseInt->Value == i * 100);
		INFO("SparseFloat value correct");
		CHECK(SparseFloat->Value == static_cast<float>(i * 5));
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.BatchAddFragmentInstances.MixedRegularAndSparse", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 15;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Create payload with both regular and sparse fragments
	UE::Mass::TMultiArray<FTestFragment_Float, FTestFragment_SparseInt> Payload;
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FTestFragment_Float RegularFloat;
		RegularFloat.Value = static_cast<float>(i * 10);

		FTestFragment_SparseInt SparseInt;
		SparseInt.Value = i * 200;

		Payload.Add(RegularFloat, SparseInt);
	}

	// Create entity collections with payload
	TArray<FStructArrayView> GenericPayload;
	GenericPayload.Reserve(Payload.GetNumArrays());
	Payload.GetAsGenericMultiArray(GenericPayload);

	TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
	FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(
		*EntityManager, Entities, FMassArchetypeEntityCollection::FoldDuplicates,
		FMassGenericPayloadView(GenericPayload), EntityCollections);

	// Build fragments bitset (mixed regular + sparse)
	FMassFragmentBitSet FragmentsAffected;
	FragmentsAffected.Add(FTestFragment_Float::StaticStruct());
	FragmentsAffected.Add(FTestFragment_SparseInt::StaticStruct());

	// Execute batch add
	EntityManager->BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected);

	// Verify all entities have both regular and sparse fragments
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FMassEntityView EntityView(*EntityManager, Entities[i]);

		// Regular fragment should be in archetype
		const FTestFragment_Float* RegularFloat = EntityView.GetFragmentDataPtr<FTestFragment_Float>();
		INFO("Regular float added");
		REQUIRE(RegularFloat != nullptr);
		INFO("Regular float value correct");
		CHECK(RegularFloat->Value == static_cast<float>(i * 10));

		// Sparse fragment should be in external storage
		const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		INFO("Sparse int added");
		REQUIRE(SparseInt != nullptr);
		INFO("Sparse int value correct");
		CHECK(SparseInt->Value == i * 200);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.BatchAddFragmentInstances.CrossChunks", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);
	const int32 NumEntities = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f); // Span 3 chunks

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

	// Create payload spanning multiple chunks
	UE::Mass::TMultiArray<FTestFragment_SparseInt> Payload;
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FTestFragment_SparseInt SparseInt;
		SparseInt.Value = i * 777;
		Payload.Add(SparseInt);
	}

	// Create entity collections
	TArray<FStructArrayView> GenericPayload;
	GenericPayload.Reserve(Payload.GetNumArrays());
	Payload.GetAsGenericMultiArray(GenericPayload);

	TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
	FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(
		*EntityManager, Entities, FMassArchetypeEntityCollection::FoldDuplicates,
		FMassGenericPayloadView(GenericPayload), EntityCollections);

	FMassFragmentBitSet FragmentsAffected;
	FragmentsAffected.Add(FTestFragment_SparseInt::StaticStruct());

	// Execute across chunks
	EntityManager->BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected);

	// Verify all entities across all chunks have correct data
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FMassEntityView EntityView(*EntityManager, Entities[i]);
		const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();

		INFO("Sparse fragment present across chunks");
		REQUIRE(SparseInt != nullptr);
		INFO("Sparse fragment value correct across chunks");
		CHECK(SparseInt->Value == i * 777);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.BatchAddFragmentInstances.OverwriteExisting", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// First, add sparse fragments with initial values
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FStructView SparseInt = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
		SparseInt.Get<FTestFragment_SparseInt>().Value = (i + 1) * 10; // Initial value
	}

	// Verify initial values
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FMassEntityView EntityView(*EntityManager, Entities[i]);
		const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		INFO("Initial sparse fragment present");
		REQUIRE(SparseInt != nullptr);
		INFO("Initial value correct");
		CHECK(SparseInt->Value == (i + 1) * 10);
	}

	// Now use BatchAddFragmentInstancesForEntities to overwrite with new values
	UE::Mass::TMultiArray<FTestFragment_SparseInt> Payload;
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FTestFragment_SparseInt NewSparseInt;
		NewSparseInt.Value = (i + 1) * 999; // New value
		Payload.Add(NewSparseInt);
	}

	TArray<FStructArrayView> GenericPayload;
	GenericPayload.Reserve(Payload.GetNumArrays());
	Payload.GetAsGenericMultiArray(GenericPayload);

	TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
	FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(
		*EntityManager, Entities, FMassArchetypeEntityCollection::FoldDuplicates,
		FMassGenericPayloadView(GenericPayload), EntityCollections);

	FMassFragmentBitSet FragmentsAffected;
	FragmentsAffected.Add(FTestFragment_SparseInt::StaticStruct());

	EntityManager->BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected);

	// Verify values were overwritten
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FMassEntityView EntityView(*EntityManager, Entities[i]);
		const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();

		INFO("Sparse fragment still present");
		REQUIRE(SparseInt != nullptr);
		INFO("Value overwritten correctly");
		CHECK(SparseInt->Value == (i + 1) * 999);
		INFO("Value changed from initial");
		CHECK(SparseInt->Value != (i + 1) * 10);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.CommandAddFragmentInstances.PureSparse", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 12;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

	// Use command to add sparse fragments
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FTestFragment_SparseInt SparseInt;
		SparseInt.Value = i * 50;

		FTestFragment_SparseFloat SparseFloat;
		SparseFloat.Value = static_cast<float>(i * 3);

		EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[i], SparseInt, SparseFloat);
	}

	// Flush commands
	EntityManager->FlushCommands();

	// Verify all entities have sparse fragments with correct values
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FMassEntityView EntityView(*EntityManager, Entities[i]);

		const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		const FTestFragment_SparseFloat* SparseFloat = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();

		INFO("Command added SparseInt");
		REQUIRE(SparseInt != nullptr);
		INFO("Command added SparseFloat");
		REQUIRE(SparseFloat != nullptr);

		INFO("Command SparseInt value correct");
		CHECK(SparseInt->Value == i * 50);
		INFO("Command SparseFloat value correct");
		CHECK(SparseFloat->Value == static_cast<float>(i * 3));
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.CommandAddFragmentInstances.MixedRegularAndSparse", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 8;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Use command with mixed regular + sparse fragments
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FTestFragment_Float RegularFloat;
		RegularFloat.Value = static_cast<float>(i * 25);

		FTestFragment_SparseInt SparseInt;
		SparseInt.Value = i * 888;

		EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[i], RegularFloat, SparseInt);
	}

	EntityManager->FlushCommands();

	// Verify both regular and sparse fragments
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FMassEntityView EntityView(*EntityManager, Entities[i]);

		const FTestFragment_Float* RegularFloat = EntityView.GetFragmentDataPtr<FTestFragment_Float>();
		const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();

		INFO("Command added regular float");
		REQUIRE(RegularFloat != nullptr);
		INFO("Command added sparse int");
		REQUIRE(SparseInt != nullptr);

		INFO("Command regular float value correct");
		CHECK(RegularFloat->Value == static_cast<float>(i * 25));
		INFO("Command sparse int value correct");
		CHECK(SparseInt->Value == i * 888);
	}
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.CommandAddFragmentInstances.CrossChunks", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
	const int32 NumChunks = 4;
	const int32 NumEntities = EntitiesPerChunk * NumChunks;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Use command across multiple chunks
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FTestFragment_SparseFloat SparseFloat;
		SparseFloat.Value = static_cast<float>(i * 11);
		EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[i], SparseFloat);
	}

	EntityManager->FlushCommands();

	// Verify across all chunks
	int32 ChunksFound = 0;
	int32 LastChunkIndex = -1;

	for (int32 i = 0; i < NumEntities; ++i)
	{
		FMassEntityView EntityView(*EntityManager, Entities[i]);
		const FTestFragment_SparseFloat* SparseFloat = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();

		INFO("Sparse fragment present across chunks via command");
		REQUIRE(SparseFloat != nullptr);
		INFO("Value correct across chunks via command");
		CHECK(SparseFloat->Value == static_cast<float>(i * 11));

		// Track chunk transitions
		int32 CurrentChunkIndex = i / EntitiesPerChunk;
		if (CurrentChunkIndex != LastChunkIndex)
		{
			++ChunksFound;
			LastChunkIndex = CurrentChunkIndex;
		}
	}

	INFO("Command processed all chunks");
	CHECK(ChunksFound == NumChunks);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.CommandAddFragmentInstances.PartialBatch", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 30;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

	// Only add sparse fragments to every 3rd entity via command
	int32 ExpectedCount = 0;
	for (int32 i = 0; i < NumEntities; i += 3)
	{
		FTestFragment_SparseInt SparseInt;
		SparseInt.Value = i * 33;
		EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[i], SparseInt);
		++ExpectedCount;
	}

	EntityManager->FlushCommands();

	// Verify only targeted entities have sparse fragments
	int32 ActualCount = 0;
	for (int32 i = 0; i < NumEntities; ++i)
	{
		FMassEntityView EntityView(*EntityManager, Entities[i]);
		const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();

		if (i % 3 == 0)
		{
			INFO("Targeted entity has sparse fragment");
			REQUIRE(SparseInt != nullptr);
			INFO("Targeted entity value correct");
			CHECK(SparseInt->Value == i * 33);
			++ActualCount;
		}
		else
		{
			INFO("Non-targeted entity has no sparse fragment");
			CHECK(SparseInt == nullptr);
		}
	}

	INFO("Correct number of entities received sparse fragments");
	CHECK(ActualCount == ExpectedCount);
}

// @todo Re-enable once FSparseElementsStorage::GetTypeStats / GetStorageStats API is implemented.
DISABLED_TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.StorageStatistics", "[Mass][Stress][Sparse][Debug]")
{
}

#endif // WITH_MASSENTITY_DEBUG

//-----------------------------------------------------------------------------
// Chunk and Archetype Bitset Tracking Tests
//-----------------------------------------------------------------------------
#if WITH_MASSENTITY_DEBUG
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ChunkBitset.AddSingle", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse element to first entity
	FStructView _ = EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());

	// Verify chunk bitset is updated
	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
	const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
	INFO("Chunk should have sparse elements");
	CHECK(Chunk.HasSparseElements());

	const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();
	INFO("Chunk bitset should contain sparse int");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ChunkBitset.RemoveSingle", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse element to first two entities
	FStructView _[] = {
		EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct())
		, EntityManager->AddSparseElementToEntity(Entities[1], FTestFragment_SparseInt::StaticStruct())
	};

	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
	const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
	const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

	INFO("Chunk bitset should contain sparse int after adds");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	// Remove from one entity - should still be in chunk bitset
	EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
	INFO("Chunk bitset should still contain sparse int (one entity remains)");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	// Remove from last entity - should be removed from chunk bitset
	EntityManager->RemoveSparseElementFromEntity(Entities[1], FTestFragment_SparseInt::StaticStruct());
	INFO("Chunk bitset should NOT contain sparse int after all removed");
	CHECK_FALSE(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ChunkBitset.BatchAdd", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 50;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Batch add sparse elements to first 20 entities
	TArray<FMassEntityHandle> BatchEntities;
	for (int32 i = 0; i < 20; ++i)
	{
		BatchEntities.Add(Entities[i]);
	}
	FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, BatchEntities, FMassArchetypeEntityCollection::NoDuplicates);
	EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());

	// Verify chunk bitset is updated
	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
	const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
	const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

	INFO("Chunk bitset should contain sparse int after batch add");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ChunkBitset.BatchRemove", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 50;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Batch add to all entities
	{
		FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, Entities, FMassArchetypeEntityCollection::NoDuplicates);
		EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
	}

	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
	const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
	const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

	INFO("Chunk bitset should contain sparse int");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	// Batch remove from half the entities - should still be in chunk bitset
	TArray<FMassEntityHandle> HalfEntities;
	for (int32 i = 0; i < 25; ++i)
	{
		HalfEntities.Add(Entities[i]);
	}
	{
		FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, HalfEntities, FMassArchetypeEntityCollection::NoDuplicates);
		EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
	}

	INFO("Chunk bitset should still contain sparse int (some entities remain)");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	// Batch remove from remaining entities - should be removed from chunk bitset
	TArray<FMassEntityHandle> RemainingEntities;
	for (int32 i = 25; i < NumEntities; ++i)
	{
		RemainingEntities.Add(Entities[i]);
	}
	{
		FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, RemainingEntities, FMassArchetypeEntityCollection::NoDuplicates);
		EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
	}

	INFO("Chunk bitset should NOT contain sparse int after all removed");
	CHECK_FALSE(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ChunkBitset.MultipleSparseTypes", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 20;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add different sparse types to different entities
	FStructView _[] = {
		EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct())
		, EntityManager->AddSparseElementToEntity(Entities[1], FTestFragment_SparseFloat::StaticStruct())
		, EntityManager->AddSparseElementToEntity(Entities[2], FTestTag_SparseA::StaticStruct())
	};

	// Verify chunk bitset contains all three types
	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
	const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
	const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

	INFO("Chunk bitset should contain sparse int");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("Chunk bitset should contain sparse float");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseFloat::StaticStruct()));
	INFO("Chunk bitset should contain sparse tag A");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestTag_SparseA::StaticStruct()));

	// Remove one type completely
	EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());

	INFO("Chunk bitset should NOT contain sparse int after removal");
	CHECK_FALSE(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
	INFO("Chunk bitset should still contain sparse float");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseFloat::StaticStruct()));
	INFO("Chunk bitset should still contain sparse tag A");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestTag_SparseA::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ChunkBitset.CrossChunk", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
	const int32 NumEntitiesPerChunk = Archetype.GetNumEntitiesPerChunk();
	int32 NumEntities = NumEntitiesPerChunk + 1;

	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse element to entities in first chunk only
	for (int32 i = 0; i < NumEntitiesPerChunk; ++i)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
	}

	// Verify first chunk has the sparse element
	const FMassArchetypeChunk& Chunk0 = Archetype.DebugGetChunk(0);
	INFO("First chunk should have sparse elements");
	CHECK(Chunk0.HasSparseElements());
	INFO("First chunk bitset should contain sparse int");
	CHECK(Chunk0.GetSparseElementsUnsafe().GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	// Verify second chunk does NOT have the sparse element
	const FMassArchetypeChunk& Chunk1 = Archetype.DebugGetChunk(1);
	INFO("Second chunk should not have sparse int in bitset");
	CHECK_FALSE((Chunk1.HasSparseElements() && Chunk1.GetSparseElementsUnsafe().GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct())));

	// Add sparse element to second chunk
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[NumEntitiesPerChunk], FTestFragment_SparseInt::StaticStruct());
	}
	INFO("Second chunk should now have sparse elements");
	CHECK(Chunk1.HasSparseElements());
	INFO("Second chunk bitset should now contain sparse int");
	CHECK(Chunk1.GetSparseElementsUnsafe().GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
}
#endif // WITH_MASSENTITY_DEBUG

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ArchetypeBitset.SingleChunk", "[Mass][Stress][Sparse]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);

	// Initially, archetype should not contain sparse elements
	INFO("Archetype should not contain sparse int initially");
	CHECK_FALSE(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

	// Add sparse element
	FStructView _ = EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());

	// Archetype bitset should now contain the sparse element
	INFO("Archetype should contain sparse int after add");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

	// Remove the sparse element from all entities
	EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());

	// Archetype bitset should no longer contain the sparse element
	INFO("Archetype should not contain sparse int after removal");
	CHECK_FALSE(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));
}

#if WITH_MASSENTITY_DEBUG
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ArchetypeBitset.MultipleChunks", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
	const int32 NumEntities = EntitiesPerChunk + 1;  // Multiple chunks
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);

	// Add sparse element to entities in different chunks
	FStructView _ [] = {
		EntityManager->AddSparseElementToEntity(Entities[10], FTestFragment_SparseInt::StaticStruct())   // Chunk 0
		, EntityManager->AddSparseElementToEntity(Entities[EntitiesPerChunk], FTestFragment_SparseFloat::StaticStruct()) // Chunk 1
	};

	// Archetype should contain both types
	INFO("Archetype should contain sparse int");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));
	INFO("Archetype should contain sparse float");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));

	// Remove sparse int from first chunk
	EntityManager->RemoveSparseElementFromEntity(Entities[10], FTestFragment_SparseInt::StaticStruct());

	INFO("Archetype should not contain sparse int after removal");
	CHECK_FALSE(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));
	INFO("Archetype should still contain sparse float");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.EntityRemoval.ChunkBitsetUpdate", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 20;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse elements to first 5 entities
	for (int32 i = 0; i < 5; ++i)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
	}

	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
	const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
	const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

	INFO("Chunk should contain sparse int");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	// Destroy entities with sparse elements one by one (but not all)
	for (int32 i = 0; i < 3; ++i)
	{
		EntityManager->DestroyEntity(Entities[i]);
	}
	EntityManager->FlushCommands();

	// Chunk should still contain sparse int (2 entities remain)
	INFO("Chunk should still contain sparse int after partial removal");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	// Destroy remaining entities with sparse elements
	for (int32 i = 3; i < 5; ++i)
	{
		EntityManager->DestroyEntity(Entities[i]);
	}
	EntityManager->FlushCommands();

	// Chunk should no longer contain sparse int
	INFO("Chunk should not contain sparse int after all removed");
	CHECK_FALSE(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.EntityRemoval.BatchDestruction", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 50;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse elements to all entities
	{
		FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, Entities, FMassArchetypeEntityCollection::NoDuplicates);
		EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
	}

	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
	const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
	const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

	INFO("Chunk should contain sparse int");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	// Batch destroy all entities
	EntityManager->BatchDestroyEntities(Entities);
	EntityManager->FlushCommands();

	// Chunk should no longer exist or be empty
	INFO("Archetype should have no entities after batch destruction");
	CHECK(Archetype.GetNumEntities() == 0);
}
#endif // WITH_MASSENTITY_DEBUG

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ArchetypeChange.BitsetTracking", "[Mass][Stress][Sparse]")
{
	REQUIRE(EntityManager);

	// Create source archetype with just Int
	FMassArchetypeCompositionDescriptor SourceComposition;
	SourceComposition.Add<FTestFragment_Int>();
	const FMassArchetypeHandle SourceArchetype = EntityManager->CreateArchetype(SourceComposition);

	// Create target archetype with Int + Float
	FMassArchetypeCompositionDescriptor TargetComposition;
	TargetComposition.Add<FTestFragment_Int>();
	TargetComposition.Add<FTestFragment_Float>();
	const FMassArchetypeHandle TargetArchetype = EntityManager->CreateArchetype(TargetComposition);

	constexpr int32 NumEntities = 20;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(SourceArchetype, NumEntities, Entities);

	// Add sparse elements to entities in source archetype
	for (int32 i = 0; i < 10; ++i)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
	}

	FMassArchetypeData& SourceArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype);
	INFO("Source archetype should contain sparse int");
	CHECK(SourceArchetypeData.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

	// Move entities to target archetype
	for (int32 i = 0; i < 10; ++i)
	{
		EntityManager->AddFragmentToEntity(Entities[i], FTestFragment_Float::StaticStruct());
	}

	FMassArchetypeData& TargetArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(TargetArchetype);

	// Target archetype should now contain sparse elements
	INFO("Target archetype should contain sparse int after move");
	CHECK(TargetArchetypeData.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

	// Source archetype should still have entities with sparse elements (remaining 10 entities)
	INFO("Source archetype should no longer contain sparse int - all sparse fragment owning entities have been moved");
	CHECK(SourceArchetypeData.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()) == false);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.Complex.BitsetAccuracy", "[Mass][Stress][Sparse]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 100;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);

	// Complex sequence of operations
	// 1. Batch add sparse int to first 50
	TArray<FMassEntityHandle> FirstHalf;
	for (int32 i = 0; i < 50; ++i)
	{
		FirstHalf.Add(Entities[i]);
	}
	{
		FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, FirstHalf, FMassArchetypeEntityCollection::NoDuplicates);
		EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
	}
	INFO("Should contain sparse int after batch add");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

	// 2. Add sparse float to entities 25-75 (overlapping)
	for (int32 i = 25; i < 75; ++i)
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseFloat::StaticStruct());
	}
	INFO("Should contain sparse float");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));

	// 3. Remove sparse int from 0-25
	TArray<FMassEntityHandle> FirstQuarter;
	for (int32 i = 0; i < 25; ++i)
	{
		FirstQuarter.Add(Entities[i]);
	}
	{
		FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, FirstQuarter, FMassArchetypeEntityCollection::NoDuplicates);
		EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
	}
	INFO("Should still contain sparse int (entities 25-49 remain)");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

	// 4. Destroy entities 40-60
	for (int32 i = 40; i < 60; ++i)
	{
		EntityManager->DestroyEntity(Entities[i]);
	}
	EntityManager->FlushCommands();

	// Both types should still be present
	INFO("Should still contain sparse int (entities 25-39 remain)");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));
	INFO("Should still contain sparse float (entities 25-39 and 60-74 remain)");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));

	// 5. Remove all remaining sparse int
	TArray<FMassEntityHandle> RemainingWithInt;
	for (int32 i = 25; i < 40; ++i)
	{
		RemainingWithInt.Add(Entities[i]);
	}
	{
		FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, RemainingWithInt, FMassArchetypeEntityCollection::NoDuplicates);
		EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
	}

	INFO("Should NOT contain sparse int after all removed");
	CHECK_FALSE(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));
	INFO("Should still contain sparse float");
	CHECK(Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));
}

#if WITH_MASSENTITY_DEBUG
TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.ReAdd.ChunkBitsetConsistency", "[Mass][Stress][Sparse][Debug]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
	const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
	const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

	// Add -> Remove -> Add cycle
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
	}
	INFO("Chunk should contain sparse int after first add");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
	INFO("Chunk should NOT contain sparse int after remove");
	CHECK_FALSE(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
	}
	INFO("Chunk should contain sparse int after re-add");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

	// Add to second entity and remove from first - bitset should remain
	{
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[1], FTestFragment_SparseInt::StaticStruct());
	}
	EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
	INFO("Chunk should still contain sparse int (second entity has it)");
	CHECK(ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
}
#endif // WITH_MASSENTITY_DEBUG

//----------------------------------------------------------------------//
// Unified API Tests - verifying Fragment/Tag API works seamlessly with Sparse Elements
//----------------------------------------------------------------------//

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.AddFragmentToEntity", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Verify entity doesn't have sparse fragment initially
	INFO("Entity should not have sparse fragment initially");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	// Use AddFragmentToEntity with sparse fragment type
	EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	// Verify sparse fragment was added
	INFO("Entity should have sparse fragment after AddFragmentToEntity");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	// Verify entity remained in same archetype (sparse doesn't change archetype)
	INFO("Entity should remain in original archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);

	// Verify we can access the sparse fragment data
	FMassEntityView EntityView(*EntityManager, Entity);
	const FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Should be able to access sparse fragment data");
	CHECK(SparseFragment != nullptr);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.AddFragmentToEntity.WithInitializer", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	constexpr int32 InitialValue = 42;
	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Use AddFragmentToEntity with initializer callback for sparse fragment
	EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct(),
		[InitialValue](void* FragmentMemory, const UScriptStruct& FragmentType)
		{
			FTestFragment_SparseInt* Fragment = static_cast<FTestFragment_SparseInt*>(FragmentMemory);
			Fragment->Value = InitialValue;
		});

	// Verify sparse fragment was added with correct value
	INFO("Entity should have sparse fragment");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	FMassEntityView EntityView(*EntityManager, Entity);
	const FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Should be able to access sparse fragment data");
	REQUIRE(SparseFragment != nullptr);
	INFO("Sparse fragment should have initialized value");
	CHECK(SparseFragment->Value == InitialValue);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.RemoveFragmentFromEntity", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add sparse fragment first
	EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	INFO("Entity should have sparse fragment after add");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	// Use RemoveFragmentFromEntity with sparse fragment type
	EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	// Verify sparse fragment was removed
	INFO("Entity should not have sparse fragment after RemoveFragmentFromEntity");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	// Verify entity remained in same archetype
	INFO("Entity should remain in original archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.AddTagToEntity", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Verify entity doesn't have sparse tag initially
	INFO("Entity should not have sparse tag initially");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

	// Use AddTagToEntity with sparse tag type
	EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());

	// Verify sparse tag was added
	INFO("Entity should have sparse tag after AddTagToEntity");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

	// Verify entity remained in same archetype (sparse doesn't change archetype)
	INFO("Entity should remain in original archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);

	// Verify via FMassEntityView
	FMassEntityView EntityView(*EntityManager, Entity);
	INFO("EntityView should report having sparse tag");
	CHECK(EntityView.HasElement<FTestTag_SparseA>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.RemoveTagFromEntity", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add sparse tag first
	EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());
	INFO("Entity should have sparse tag after add");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

	// Use RemoveTagFromEntity with sparse tag type
	EntityManager->RemoveTagFromEntity(Entity, FTestTag_SparseA::StaticStruct());

	// Verify sparse tag was removed
	INFO("Entity should not have sparse tag after RemoveTagFromEntity");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

	// Verify entity remained in same archetype
	INFO("Entity should remain in original archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.AddElementToEntity.SparseFragment", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Use generic AddElementToEntity with sparse fragment type
	EntityManager->AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	// Verify sparse fragment was added
	INFO("Entity should have sparse fragment after AddElementToEntity");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	// Verify archetype unchanged
	INFO("Entity should remain in original archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.AddElementToEntity.SparseTag", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Use generic AddElementToEntity with sparse tag type
	EntityManager->AddElementToEntity(Entity, FTestTag_SparseA::StaticStruct());

	// Verify sparse tag was added
	INFO("Entity should have sparse tag after AddElementToEntity");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

	// Verify archetype unchanged
	INFO("Entity should remain in original archetype");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == IntsArchetype);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.RemoveElementFromEntity.SparseFragment", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add then remove sparse fragment using generic API
	EntityManager->AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	INFO("Entity should have sparse fragment");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

	EntityManager->RemoveElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	// Verify sparse fragment was removed
	INFO("Entity should not have sparse fragment after RemoveElementFromEntity");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.RemoveElementFromEntity.SparseTag", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add then remove sparse tag using generic API
	EntityManager->AddElementToEntity(Entity, FTestTag_SparseA::StaticStruct());
	INFO("Entity should have sparse tag");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

	EntityManager->RemoveElementFromEntity(Entity, FTestTag_SparseA::StaticStruct());

	// Verify sparse tag was removed
	INFO("Entity should not have sparse tag after RemoveElementFromEntity");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.DoesEntityHaveElement.AllTypes", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Verify none present initially
	INFO("Should not have sparse int initially");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
	INFO("Should not have sparse float initially");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseFloat::StaticStruct()));
	INFO("Should not have sparse tag A initially");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
	INFO("Should not have sparse tag B initially");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));

	// Add all sparse element types
	EntityManager->AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	EntityManager->AddElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
	EntityManager->AddElementToEntity(Entity, FTestTag_SparseA::StaticStruct());
	EntityManager->AddElementToEntity(Entity, FTestTag_SparseB::StaticStruct());

	// Verify all present
	INFO("Should have sparse int");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
	INFO("Should have sparse float");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseFloat::StaticStruct()));
	INFO("Should have sparse tag A");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
	INFO("Should have sparse tag B");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));

	// Also verify template version
	INFO("Template DoesEntityHaveElement should work for sparse int");
	CHECK(EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(Entity));
	INFO("Template DoesEntityHaveElement should work for sparse tag");
	CHECK(EntityManager->DoesEntityHaveElement<FTestTag_SparseA>(Entity));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.EntityView.HasElement", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
	FMassEntityView EntityView(*EntityManager, Entity);

	// Verify HasElement returns false initially for sparse elements
	INFO("EntityView should not report sparse fragment initially");
	CHECK_FALSE(EntityView.HasElement<FTestFragment_SparseInt>());
	INFO("EntityView should not report sparse tag initially");
	CHECK_FALSE(EntityView.HasElement<FTestTag_SparseA>());

	// Add sparse elements
	EntityManager->AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	EntityManager->AddElementToEntity(Entity, FTestTag_SparseA::StaticStruct());

	// Verify HasElement returns true
	INFO("EntityView should report sparse fragment after add");
	CHECK(EntityView.HasElement<FTestFragment_SparseInt>());
	INFO("EntityView should report sparse tag after add");
	CHECK(EntityView.HasElement<FTestTag_SparseA>());

	// Test with EIncludeSparseElements parameter
	INFO("HasElement with IncludeSparseElements::Yes should return true");
	CHECK(EntityView.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::Yes));
	INFO("HasElement with IncludeSparseElements::No should return false for sparse element");
	CHECK_FALSE(EntityView.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::No));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.MixedRegularAndSparse", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add regular fragment (changes archetype)
	EntityManager->AddFragmentToEntity(Entity, FTestFragment_Float::StaticStruct());
	const FMassArchetypeHandle NewArchetype = EntityManager->GetArchetypeForEntity(Entity);
	INFO("Archetype should change after adding regular fragment");
	CHECK(NewArchetype != IntsArchetype);

	// Add sparse fragment (should not change archetype)
	EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	INFO("Archetype should not change after adding sparse fragment");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == NewArchetype);

	// Add regular tag (changes archetype)
	EntityManager->AddTagToEntity(Entity, FTestTag_A::StaticStruct());
	const FMassArchetypeHandle ArchetypeWithTag = EntityManager->GetArchetypeForEntity(Entity);
	INFO("Archetype should change after adding regular tag");
	CHECK(ArchetypeWithTag != NewArchetype);

	// Add sparse tag (should not change archetype)
	EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());
	INFO("Archetype should not change after adding sparse tag");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == ArchetypeWithTag);

	// Verify all elements present
	INFO("Should have regular int fragment (from original archetype)");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
	INFO("Should have regular float fragment");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
	INFO("Should have sparse int fragment");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
	INFO("Should have regular tag A");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
	INFO("Should have sparse tag A");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

	// Remove sparse elements - archetype should not change
	EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	EntityManager->RemoveTagFromEntity(Entity, FTestTag_SparseA::StaticStruct());
	INFO("Archetype should not change after removing sparse elements");
	CHECK(EntityManager->GetArchetypeForEntity(Entity) == ArchetypeWithTag);

	// Verify sparse elements removed
	INFO("Should not have sparse int fragment after removal");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
	INFO("Should not have sparse tag A after removal");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

	// Verify regular elements still present
	INFO("Should still have regular float fragment");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
	INFO("Should still have regular tag A");
	CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.BatchOperations", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	constexpr int32 NumEntities = 10;
	TArray<FMassEntityHandle> Entities;
	EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	// Add sparse fragments to all entities using unified API
	for (const FMassEntityHandle& Entity : Entities)
	{
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());
	}

	// Verify all entities have sparse elements
	for (const FMassEntityHandle& Entity : Entities)
	{
		INFO("Each entity should have sparse fragment");
		CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		INFO("Each entity should have sparse tag");
		CHECK(EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
	}

	// Remove from half
	for (int32 i = 0; i < NumEntities / 2; ++i)
	{
		EntityManager->RemoveFragmentFromEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
		EntityManager->RemoveTagFromEntity(Entities[i], FTestTag_SparseA::StaticStruct());
	}

	// Verify first half don't have, second half still have
	for (int32 i = 0; i < NumEntities; ++i)
	{
		if (i < NumEntities / 2)
		{
			INFO("First half should not have sparse fragment");
			CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
			INFO("First half should not have sparse tag");
			CHECK_FALSE(EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));
		}
		else
		{
			INFO("Second half should have sparse fragment");
			CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
			INFO("Second half should have sparse tag");
			CHECK(EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));
		}
	}
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.RemoveNonExistent", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Try to remove sparse elements that were never added - should not crash
	EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());
	EntityManager->RemoveTagFromEntity(Entity, FTestTag_SparseA::StaticStruct());
	EntityManager->RemoveElementFromEntity(Entity, FTestFragment_SparseFloat::StaticStruct());

	// Verify entity is still valid
	INFO("Entity should still be valid after removing non-existent sparse elements");
	CHECK(EntityManager->IsEntityValid(Entity));
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.AddDuplicate", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Add sparse fragment
	EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	// Set a value
	FMassEntityView EntityView(*EntityManager, Entity);
	FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Should have sparse fragment");
	REQUIRE(SparseFragment != nullptr);
	SparseFragment->Value = 123;

	// Try to add the same sparse fragment again
	EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	// Verify value is preserved (not reset)
	SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
	INFO("Should still have sparse fragment");
	REQUIRE(SparseFragment != nullptr);
	INFO("Sparse fragment value should be preserved after duplicate add");
	CHECK(SparseFragment->Value == 123);
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.EntityView.HasFragment", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
	FMassEntityView EntityView(*EntityManager, Entity);

	// Verify HasFragment works for sparse fragments
	INFO("HasFragment should return false for absent sparse fragment");
	CHECK_FALSE(EntityView.HasFragment(FTestFragment_SparseInt::StaticStruct()));

	EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

	INFO("HasFragment should return true for present sparse fragment");
	CHECK(EntityView.HasFragment(FTestFragment_SparseInt::StaticStruct()));

	// Template version
	INFO("Template HasFragment should work for sparse fragment");
	CHECK(EntityView.HasFragment<FTestFragment_SparseInt>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.EntityView.HasTag", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
	FMassEntityView EntityView(*EntityManager, Entity);

	// Verify HasTag works for sparse tags
	INFO("HasTag should return false for absent sparse tag");
	CHECK_FALSE(EntityView.HasTag(FTestTag_SparseA::StaticStruct()));

	EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());

	INFO("HasTag should return true for present sparse tag");
	CHECK(EntityView.HasTag(FTestTag_SparseA::StaticStruct()));

	// Template version
	INFO("Template HasTag should work for sparse tag");
	CHECK(EntityView.HasTag<FTestTag_SparseA>());
}


TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass.Stress.Sparse.UnifiedAPI.FragmentListOperations", "[Mass][Stress][Sparse][UnifiedAPI]")
{
	REQUIRE(EntityManager);

	const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

	// Create array of sparse fragment types for RemoveFragmentListFromEntity
	TArray<const UScriptStruct*> SparseFragmentTypes;
	SparseFragmentTypes.Add(FTestFragment_SparseInt::StaticStruct());
	SparseFragmentTypes.Add(FTestFragment_SparseFloat::StaticStruct());

	// Add sparse fragments first
	for (const UScriptStruct* Type : SparseFragmentTypes)
	{
		EntityManager->AddFragmentToEntity(Entity, Type);
	}

	// Verify all added
	INFO("Should have sparse int");
	CHECK(EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(Entity));
	INFO("Should have sparse float");
	CHECK(EntityManager->DoesEntityHaveElement<FTestFragment_SparseFloat>(Entity));

	// Remove using RemoveFragmentListFromEntity
	EntityManager->RemoveFragmentListFromEntity(Entity, SparseFragmentTypes);

	// Verify all removed
	INFO("Should not have sparse int after list removal");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(Entity));
	INFO("Should not have sparse float after list removal");
	CHECK_FALSE(EntityManager->DoesEntityHaveElement<FTestFragment_SparseFloat>(Entity));
}

} // namespace UE::Mass::LLT

#undef WITH_ENTITY_MOVING_AFFECTING_COMPOSITION

UE_ENABLE_OPTIMIZATION_SHIP
