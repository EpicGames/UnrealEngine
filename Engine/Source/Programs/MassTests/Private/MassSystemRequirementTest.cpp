// Copyright Epic Games, Inc. All Rights Reserved.

// NOTE: MassSystemRequirementTest requires UWorld, GEngine, ULocalPlayer, and UGameInstance subsystems.
// These tests cannot run in standalone LLT programs (no game world).
// They remain in the Automation Framework (MassEntityTestSuite) only.
//
// Skipped tests:
// - FGenericSubsystemAccessAPI ("System.Mass.Query.System.AccessAPI.Generic")
// - FGameSubsystemAccessAPI ("System.Mass.Query.System.AccessAPI.Game")
