// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformMisc.h"
#include "Logging/LogSuppressionInterface.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "TestCommon/Initialization.h"

#include "MassEntityLLTFixture.h"

#include "TestHarness.h"

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	FCommandLine::Append(TEXT(" -nullrhi -unattended -LogCmds=\"global Fatal, LogMass Log, LogMassEntity Log\""));

	FPlatformMisc::EngineDir();

	// Apply -LogCmds BEFORE InitAll to suppress engine log noise.
	// UE_LOG routes Error/Warning/Display through GWarn (FFeedbackContextAnsi) which
	// prints directly to stdout via wprintf — bypassing GLog and any backlog buffering.
	// The only way to suppress these messages is at the UE_LOG macro level via
	// per-category verbosity thresholds set by ProcessConfigAndCommandLine.
	FLogSuppressionInterface::Get().ProcessConfigAndCommandLine();

	InitAll(true, true);

	for (const FName ModuleToLoad : {TEXT("MassEntity")})
	{
		FModuleManager::Get().LoadModule(ModuleToLoad);
	}
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
}

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::Entity::CreateEntityManager", "[Mass][Entity][Smoke]")
{
	REQUIRE(EntityManager);
	// Verify fixture initialization succeeded by checking that entity operations work
	FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
	CHECK(EntityManager->IsEntityValid(Entity));
}

} // namespace UE::Mass::LLT
