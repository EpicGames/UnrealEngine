// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "TypeBitSetBuilder.h"
#include "MassBitSetRegistry.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

//-----------------------------------------------------------------------------
// Templated helper for testing both Builder (wrapping existing set) and Factory (standalone)
//-----------------------------------------------------------------------------
template<typename Traits, typename TBuilderType>
struct FBitSetBuilderTest
{
	using FTestElement_A = typename Traits::FTestElement_A;
	using FTestElement_B = typename Traits::FTestElement_B;
	using FTestElement_C = typename Traits::FTestElement_C;

	typename Traits::FBitRegistry BitRegistry;

	FBitSetBuilderTest()
	{
		BitRegistry.RegisterType(FTestElement_A::StaticStruct());
		BitRegistry.RegisterType(FTestElement_B::StaticStruct());
		BitRegistry.RegisterType(FTestElement_C::StaticStruct());
	}

	template<typename T>
	void MakeBuilderAndTest();
};

template<typename Traits, typename TBuilderType>
template<typename T>
void FBitSetBuilderTest<Traits, TBuilderType>::MakeBuilderAndTest()
{
	// default: no specialization
}

// Fragment Traits
struct FFragmentTraits
{
	using FBitRegistry = FFragmentBitRegistry;
	using FBitSet = FMassFragmentBitSet_WIP;
	using FBitSetBuilder = FFragmentBitSetBuilder;
	using FBitSetReader = FFragmentBitSetReader;
	using FBitSetFactory = FFragmentBitSetFactory;
	using FTestElement_A = FTestFragment_Float;
	using FTestElement_B = FTestFragment_Int;
	using FTestElement_C = FTestFragment_Bool;
};

// Tag Traits
struct FTagTraits
{
	using FBitRegistry = FTagBitRegistry;
	using FBitSet = FMassTagBitSet_WIP;
	using FBitSetBuilder = FTagBitSetBuilder;
	using FBitSetReader = FTagBitSetReader;
	using FBitSetFactory = FTagBitSetFactory;
	using FTestElement_A = FTestTag_A;
	using FTestElement_B = FTestTag_B;
	using FTestElement_C = FTestTag_C;
};

template<typename Traits>
void RunBitSetBuilderScenario(typename Traits::FBitRegistry& BitRegistry, auto& BitSetBuilder)
{
	using FTestElement_A = typename Traits::FTestElement_A;
	using FTestElement_B = typename Traits::FTestElement_B;
	using FTestElement_C = typename Traits::FTestElement_C;

	BitSetBuilder.template Add<FTestElement_B>();

	typename Traits::FBitSetReader BitSetReader = BitSetBuilder;
	CHECK(BitSetReader.template Contains<FTestElement_B>() == BitSetReader.Contains(FTestElement_B::StaticStruct()));
	CHECK(BitSetReader.template Contains<FTestElement_A>() == BitSetReader.Contains(FTestElement_A::StaticStruct()));
	CHECK(BitSetReader.template Contains<FTestElement_B>());
	CHECK_FALSE(BitSetReader.template Contains<FTestElement_A>());

	const typename Traits::FBitSet BitSetCopy = BitSetBuilder;

	BitSetBuilder.template Add<FTestElement_B>();
	CHECK(BitSetCopy == typename Traits::FBitSet(BitSetBuilder));

	BitSetBuilder.template Add<FTestElement_C>();
	CHECK_FALSE(BitSetCopy == typename Traits::FBitSet(BitSetBuilder));

	BitSetBuilder.template Remove<FTestElement_C>();
	CHECK(BitSetCopy == typename Traits::FBitSet(BitSetBuilder));
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::BitSetBuilder::Wrapper::Fragments", "[Mass][BitSetBuilder]")
{
	FFragmentTraits::FBitRegistry BitRegistry;
	BitRegistry.RegisterType(FTestFragment_Float::StaticStruct());
	BitRegistry.RegisterType(FTestFragment_Int::StaticStruct());
	BitRegistry.RegisterType(FTestFragment_Bool::StaticStruct());

	FFragmentTraits::FBitSet BitSet;
	FFragmentTraits::FBitSetBuilder BitSetBuilder = BitRegistry.MakeBuilder(BitSet);
	RunBitSetBuilderScenario<FFragmentTraits>(BitRegistry, BitSetBuilder);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::BitSetBuilder::Standalone::Fragments", "[Mass][BitSetBuilder]")
{
	FFragmentTraits::FBitRegistry BitRegistry;
	BitRegistry.RegisterType(FTestFragment_Float::StaticStruct());
	BitRegistry.RegisterType(FTestFragment_Int::StaticStruct());
	BitRegistry.RegisterType(FTestFragment_Bool::StaticStruct());

	FFragmentTraits::FBitSetFactory BitSetFactory = BitRegistry.MakeBuilder();
	RunBitSetBuilderScenario<FFragmentTraits>(BitRegistry, BitSetFactory);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::BitSetBuilder::Wrapper::Tags", "[Mass][BitSetBuilder]")
{
	FTagTraits::FBitRegistry BitRegistry;
	BitRegistry.RegisterType(FTestTag_A::StaticStruct());
	BitRegistry.RegisterType(FTestTag_B::StaticStruct());
	BitRegistry.RegisterType(FTestTag_C::StaticStruct());

	FTagTraits::FBitSet BitSet;
	FTagTraits::FBitSetBuilder BitSetBuilder = BitRegistry.MakeBuilder(BitSet);
	RunBitSetBuilderScenario<FTagTraits>(BitRegistry, BitSetBuilder);
}

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::BitSetBuilder::Standalone::Tags", "[Mass][BitSetBuilder]")
{
	FTagTraits::FBitRegistry BitRegistry;
	BitRegistry.RegisterType(FTestTag_A::StaticStruct());
	BitRegistry.RegisterType(FTestTag_B::StaticStruct());
	BitRegistry.RegisterType(FTestTag_C::StaticStruct());

	FTagTraits::FBitSetFactory BitSetFactory = BitRegistry.MakeBuilder();
	RunBitSetBuilderScenario<FTagTraits>(BitRegistry, BitSetFactory);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
