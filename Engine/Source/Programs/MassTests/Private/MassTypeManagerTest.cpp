// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityLLTProcessors.h"
#include "MassEntityLLTTypes.h"
#include "MassTypeManager.h"

#include "TestHarness.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTEntityFixture, "Mass::TypeManager::StaticSubsystem", "[Mass][TypeManager]")
{
	const FSubsystemTypeTraits TestSubsystemTraits = FSubsystemTypeTraits::Make<UMassLLTWorldSubsystem>();

	INFO("Subsystem bGameThreadOnly value");
	CHECK(TestSubsystemTraits.bGameThreadOnly == TMassExternalSubsystemTraits<UMassLLTWorldSubsystem>::GameThreadOnly);
	INFO("Subsystem ThreadSafeWrite value");
	CHECK(TestSubsystemTraits.bThreadSafeWrite == TMassExternalSubsystemTraits<UMassLLTWorldSubsystem>::ThreadSafeWrite);

	const FSharedFragmentTypeTraits TestSharedFragmentTraits = FSharedFragmentTypeTraits::Make<FTestSharedFragment_Int>();
	INFO("Shared fragment bGameThreadOnly value");
	CHECK(TestSubsystemTraits.bGameThreadOnly == TMassExternalSubsystemTraits<UMassLLTWorldSubsystem>::GameThreadOnly);
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
