// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityLLTFixture.h"
#include "MassEntityUtils.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::LLT
{

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Utils::MultiSort", "[Mass][Utils]")
{
	TArray<float> ValuesF = { 1,2,3,4,5,6,7,8,9,10 };
	TArray<bool> ValuesB = { 1,0,1,0,1,0,1,0,1,0 };
	TArray<int8> ValuesU = { -1,2,-3,4,-5,6,-7,8,-9,10 };

	UE::Mass::Utils::AbstractSort(ValuesU.Num(), [&ValuesU](const int32 LHS, const int32 RHS)
		{
			return ValuesU[LHS] < ValuesU[RHS];
		}
		, [&ValuesF, &ValuesB, &ValuesU](const int32 A, const int32 B)
		{
			Swap(ValuesF[A], ValuesF[B]);
			Swap(ValuesB[A], ValuesB[B]);
			Swap(ValuesU[A], ValuesU[B]);
		});

	for (int32 Index = 0; Index < ValuesF.Num(); ++Index)
	{
		CHECK(static_cast<float>(FMath::Abs(ValuesU[Index])) == ValuesF[Index]);
		CHECK((ValuesU[Index] < 0) == ValuesB[Index]);
	}
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Utils::MultiSort_GenericArray", "[Mass][Utils]")
{
	TArray<FTestFragment_Float> ValuesF = { 1,2,3,4,5,6,7,8,9,10 };
	TArray<FTestFragment_Bool> ValuesB = { 1,0,1,0,1,0,1,0,1,0 };
	TArray<int8> ValuesU = { -1,2,-3,4,-5,6,-7,8,-9,10 };
	FStructArrayView ViewF(ValuesF);
	FStructArrayView ViewB(ValuesB);

	UE::Mass::Utils::AbstractSort(ValuesU.Num(), [&ValuesU](const int32 LHS, const int32 RHS)
		{
			return ValuesU[LHS] < ValuesU[RHS];
		}
		, [&ViewF, &ViewB, &ValuesU](const int32 A, const int32 B)
		{
			Swap(ValuesU[A], ValuesU[B]);
			ViewF.Swap(A, B);
			ViewB.Swap(A, B);
		});

	for (int32 Index = 0; Index < ValuesF.Num(); ++Index)
	{
		CHECK(static_cast<float>(FMath::Abs(ValuesU[Index])) == ViewF.GetAt<const FTestFragment_Float>(Index).Value);
		CHECK((ValuesU[Index] < 0) == ViewB.GetAt<const FTestFragment_Bool>(Index).bValue);
	}
}

TEST_CASE_METHOD(FMassLLTFixture, "Mass::Utils::MultiSort_Payload", "[Mass][Utils]")
{
	TArray<FTestFragment_Float> ValuesF = { 1,2,3,4,5,6,7,8,9,10 };
	TArray<FTestFragment_Bool> ValuesB = { 1,0,1,0,1,0,1,0,1,0 };
	TArray<int8> ValuesU = { -1,2,-3,4,-5,6,-7,8,-9,10 };
	FStructArrayView ViewF(ValuesF);
	FStructArrayView ViewB(ValuesB);
	TArray<FStructArrayView> PayloadArray = { ViewF , ViewB };
	FMassGenericPayloadView Payload(PayloadArray);

	UE::Mass::Utils::AbstractSort(ValuesU.Num(), [&ValuesU](const int32 LHS, const int32 RHS)
		{
			return ValuesU[LHS] < ValuesU[RHS];
		}
		, [Payload, &ValuesU](const int32 A, const int32 B) mutable
		{
			Swap(ValuesU[A], ValuesU[B]);
			Payload.Swap(A, B);
		});

	for (int32 Index = 0; Index < ValuesF.Num(); ++Index)
	{
		CHECK(static_cast<float>(FMath::Abs(ValuesU[Index])) == ViewF.GetAt<const FTestFragment_Float>(Index).Value);
		CHECK((ValuesU[Index] < 0) == ViewB.GetAt<const FTestFragment_Bool>(Index).bValue);
	}
}

} // namespace UE::Mass::LLT

UE_ENABLE_OPTIMIZATION_SHIP
